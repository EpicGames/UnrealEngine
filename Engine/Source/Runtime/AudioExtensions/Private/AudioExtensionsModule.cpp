// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "AudioExtensionsLog.h"
#include "AudioExtentionsModule.h"
#include "HAL/LowLevelMemStats.h"
#include "IAudioExtensionPlugin.h"
#include "Algo/ForEach.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"

DECLARE_LLM_MEMORY_STAT(TEXT("AudioSpatializationPlugins"), STAT_AudioSpatializationPluginsLLM, STATGROUP_LLMFULL);
LLM_DEFINE_TAG(Audio_SpatializationPlugins, NAME_None, TEXT("Audio"), GET_STATFNAME(STAT_AudioSpatializationPluginsLLM), GET_STATFNAME(STAT_AudioSummaryLLM));

DEFINE_LOG_CATEGORY(LogAudioExtensions)

FAudioExtensionsModule* FAudioExtensionsModule::Get()
{
	// This a fast check if loaded and will load the module if not.
	return FModuleManager::Get().LoadModulePtr<FAudioExtensionsModule>(TEXT("AudioExtensions"));
}

void FAudioExtensionsModule::StartupModule()
{
	// Load Platform specific Audio feature modules. 
	TArray<FName> Platforms = FDataDrivenPlatformInfoRegistry::GetSortedPlatformNames(EPlatformInfoType::TruePlatformsOnly);
	Algo::ForEach(Platforms,[](const FName InName)
	{
		// Load any modules that are named <PlatformName>AudioFeatures, which will load hidden platforms.
		// This equivalent, but slightly safer that using the wildcard "*AudioFeatures" but without the risk
		// of accidentally bringing in user modules.
		const FString PlatformModuleName = InName.ToString() + TEXT("AudioFeatures");

		// Most of these calls will fail, but will do so quickly as the module manager already knows about all
		// possible modules, so this is a quick lookup.
		FModuleManager::Get().LoadModule(*PlatformModuleName);
	});
		
	FModuleManager::Get().LoadModuleChecked(TEXT("SignalProcessing"));
	FModuleManager::Get().LoadModuleChecked(TEXT("AudioMixerCore"));
}

IMPLEMENT_MODULE(FAudioExtensionsModule, AudioExtensions);
