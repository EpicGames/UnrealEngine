// Copyright Epic Games, Inc. All Rights Reserved.
#include "AudioParameter.h"

#include "Algo/BinarySearch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioParameter)

namespace Audio
{
	namespace ParameterPrivate
	{
		template <typename T>
		void SetOrMergeArray(const TArray<T>& InArray, TArray<T>& OutArray, bool bInMerge)
		{
			if (bInMerge)
			{
				OutArray.Append(InArray);
			}
			else
			{
				OutArray = InArray;
			}
		}
	} // ParameterPrivate

	const FString FParameterPath::NamespaceDelimiter = TEXT(AUDIO_PARAMETER_NAMESPACE_PATH_DELIMITER);

	FName FParameterPath::CombineNames(FName InLeft, FName InRight)
	{
		if (InLeft.IsNone())
		{
			return InRight;
		}

		const FString FullName = FString::Join(TArray<FString>({ *InLeft.ToString(), *InRight.ToString() }), *NamespaceDelimiter);
		return FName(FullName);
	}

	void FParameterPath::SplitName(FName InFullName, FName& OutNamespace, FName& OutParameterName)
	{
		FString FullName = InFullName.ToString();
		const int32 IndexOfDelim = FullName.Find(NamespaceDelimiter, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		if (IndexOfDelim != INDEX_NONE)
		{
			OutNamespace = FName(*FullName.Left(IndexOfDelim));
			OutParameterName = FName(*FullName.RightChop(IndexOfDelim + 1));
		}
		else
		{
			OutNamespace = FName();
			OutParameterName = InFullName;
		}
	}
} // namespace Audio

void FAudioParameter::Merge(const FAudioParameter& InParameter, bool bInTakeName, bool bInTakeType, bool bInMergeArrayTypes)
{
	if (bInTakeName)
	{
		ParamName = InParameter.ParamName;
	}

	if (bInTakeType)
	{
		ParamType = InParameter.ParamType;
	}

	switch (InParameter.ParamType)
	{
		case EAudioParameterType::Boolean:
		{
			BoolParam = InParameter.BoolParam;
		}
		break;

		case EAudioParameterType::BooleanArray:
		{
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayBoolParam, ArrayBoolParam, bInMergeArrayTypes);
		}
		break;

		case EAudioParameterType::Float:
		{
			FloatParam = InParameter.FloatParam;
		}
		break;

		case EAudioParameterType::FloatArray:
		{
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayFloatParam, ArrayFloatParam, bInMergeArrayTypes);
		}
		break;

		case EAudioParameterType::Integer:
		case EAudioParameterType::NoneArray:
		{
			if (bInMergeArrayTypes)
			{
				IntParam += InParameter.IntParam;
			}
			else
			{
				IntParam = InParameter.IntParam;
			}
		}
		break;

		case EAudioParameterType::IntegerArray:
		{
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayIntParam, ArrayIntParam, bInMergeArrayTypes);
		}
		break;

		case EAudioParameterType::None:
		{
			FloatParam = InParameter.FloatParam;
			BoolParam = InParameter.BoolParam;
			IntParam = InParameter.IntParam;
			ObjectParam = InParameter.ObjectParam;
			StringParam = InParameter.StringParam;

			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayBoolParam, ArrayBoolParam, bInMergeArrayTypes);
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayFloatParam, ArrayFloatParam, bInMergeArrayTypes);
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayIntParam, ArrayIntParam, bInMergeArrayTypes);
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayObjectParam, ArrayObjectParam, bInMergeArrayTypes);
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayStringParam, ArrayStringParam, bInMergeArrayTypes);

			if (!bInMergeArrayTypes)
			{
				ObjectProxies.Reset();
			}

			for (const TSharedPtr<Audio::IProxyData>& ProxyPtr : InParameter.ObjectProxies)
			{
				ObjectProxies.Emplace(ProxyPtr);
			}
		}
		break;

		case EAudioParameterType::Object:
		{
			ObjectParam = InParameter.ObjectParam;

			ObjectProxies.Reset();
			for (const TSharedPtr<Audio::IProxyData>& ProxyPtr : InParameter.ObjectProxies)
			{
				ObjectProxies.Emplace(ProxyPtr);
			}
		}
		break;

		case EAudioParameterType::ObjectArray:
		{
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayObjectParam, ArrayObjectParam, bInMergeArrayTypes);

			if (!bInMergeArrayTypes)
			{
				ObjectProxies.Reset();
			}

			for (const TSharedPtr<Audio::IProxyData>& ProxyPtr : InParameter.ObjectProxies)
			{
				ObjectProxies.Emplace(ProxyPtr);
			}
		}
		break;

		case EAudioParameterType::String:
		{
			StringParam = InParameter.StringParam;
		}
		break;

		case EAudioParameterType::StringArray:
		{
			Audio::ParameterPrivate::SetOrMergeArray(InParameter.ArrayStringParam, ArrayStringParam, bInMergeArrayTypes);
		}
		break;

		default:
			break;
	}
}

void FAudioParameter::Merge(TArray<FAudioParameter>&& InParams, TArray<FAudioParameter>& OutParams)
{
	if (InParams.IsEmpty())
	{
		return;
	}

	if (OutParams.IsEmpty())
	{
		OutParams = MoveTemp(InParams);
		return;
	}

	auto SortParamsPredicate = [](const FAudioParameter& A, const FAudioParameter& B) { return A.ParamName.FastLess(B.ParamName); };
	OutParams.Sort(SortParamsPredicate);

	for (FAudioParameter& NewParam : InParams)
	{
		const int32 ExistingElementIndex = Algo::LowerBound(OutParams, NewParam, SortParamsPredicate);
		if (OutParams.IsValidIndex(ExistingElementIndex))
		{
			FAudioParameter& ExistingParam = OutParams[ExistingElementIndex];
			if (ExistingParam.ParamName == NewParam.ParamName)
			{
				ExistingParam.Merge(NewParam);
				continue;
			}
		}

		OutParams.Emplace(MoveTemp(NewParam));
	}
}

