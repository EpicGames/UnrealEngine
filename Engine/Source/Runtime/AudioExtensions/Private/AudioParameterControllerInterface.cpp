// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioParameterControllerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioParameterControllerInterface)


UAudioParameterControllerInterface::UAudioParameterControllerInterface(FObjectInitializer const& InObjectInitializer)
	: UInterface(InObjectInitializer)
{
}

