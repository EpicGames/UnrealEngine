// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioEndpoint.h"

#include "AudioExtentionsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAudioEndpoint)

DEFINE_LOG_CATEGORY(LogAudioEndpoints);

TUniquePtr<IAudioEndpointSettingsProxy> UDummyEndpointSettings::GetProxy() const
{
	FDummyEndpointSettingsProxy* Settings = new FDummyEndpointSettingsProxy();

	return TUniquePtr<IAudioEndpointSettingsProxy>(Settings);
}

Audio::FPatchInput IAudioEndpoint::PatchNewInput(float ExpectedDurationPerRender, float& OutSampleRate, int32& OutNumChannels)
{
	OutSampleRate = GetSampleRate();
	OutNumChannels = GetNumChannels();

	// For average case scenarios, we need to buffer at least the sum of the number of input frames and the number of output frames per callback.
	// A good heuristic for doing this while retaining some extra space in the buffer is doubling the max of these two values.
	int32 NumSamplesToBuffer = FMath::CeilToInt(ExpectedDurationPerRender * OutNumChannels * OutSampleRate);
	if (EndpointRequiresCallback())
	{
		NumSamplesToBuffer = FMath::Max(GetDesiredNumFrames() * OutNumChannels, NumSamplesToBuffer) * 2;
	}
	else
	{
		NumSamplesToBuffer *= 2;
	}

	return PatchMixer.AddNewInput(NumSamplesToBuffer, 1.0f);
}

void IAudioEndpoint::SetNewSettings(TUniquePtr<IAudioEndpointSettingsProxy>&& InNewSettings)
{
	FScopeLock ScopeLock(&CurrentSettingsCriticalSection);

	CurrentSettings = MoveTemp(InNewSettings);
}

void IAudioEndpoint::ProcessAudioIfNeccessary()
{
	const bool bShouldExecuteCallback = !RenderCallback.IsValid() && EndpointRequiresCallback();
	if (bShouldExecuteCallback)
	{
		RunCallbackSynchronously();
	}
}

bool IAudioEndpoint::IsImplemented()
{
	return false;
}

float IAudioEndpoint::GetSampleRate() const
{
	return 0;
}

int32 IAudioEndpoint::GetNumChannels() const
{
	return 0;
}

int32 IAudioEndpoint::PopAudio(float* OutAudio, int32 NumSamples)
{
	check(OutAudio);
	return PatchMixer.PopAudio(OutAudio, NumSamples, false);
}

void IAudioEndpoint::PollSettings(TFunctionRef<void(const IAudioEndpointSettingsProxy*)> SettingsCallback)
{
	FScopeLock ScopeLock(&CurrentSettingsCriticalSection);
	SettingsCallback(CurrentSettings.Get());
}

void IAudioEndpoint::DisconnectAllInputs()
{
	PatchMixer.DisconnectAllInputs();
}

void IAudioEndpoint::StartRunningAsyncCallback()
{
	if (!ensureMsgf(GetSampleRate() > 0.0f, TEXT("Invalid sample rate returned!")))
	{
		return;
	}

	float CallbackDuration = ((float)GetDesiredNumFrames()) / GetSampleRate();

	RenderCallback.Reset(new Audio::FMixerNullCallback(CallbackDuration, [&]()
	{
		RunCallbackSynchronously();
	}));
}

void IAudioEndpoint::StopRunningAsyncCallback()
{
	RenderCallback.Reset();
}

void IAudioEndpoint::RunCallbackSynchronously()
{
	const int32 NumSamplesToBuffer = GetDesiredNumFrames() * GetNumChannels();

	BufferForRenderCallback.Reset();
	BufferForRenderCallback.AddUninitialized(NumSamplesToBuffer);

	while (PatchMixer.MaxNumberOfSamplesThatCanBePopped() >= NumSamplesToBuffer)
	{
		int32 PopResult = PatchMixer.PopAudio(BufferForRenderCallback.GetData(), BufferForRenderCallback.Num(), false);
		check(PopResult == BufferForRenderCallback.Num() || PopResult < 0);

		const TArrayView<const float> PoppedAudio = TArrayView<const float>(BufferForRenderCallback);

		auto CallbackWithSettings = [&, PoppedAudio](const IAudioEndpointSettingsProxy* InSettings)
		{
			if (!OnAudioCallback(PoppedAudio, GetNumChannels(), InSettings))
			{
				DisconnectAllInputs();
			}
		};

		PollSettings(CallbackWithSettings);
	}
}

FName IAudioEndpointFactory::GetEndpointTypeName() 
{ 
	return FName(); 
}

FName IAudioEndpointFactory::GetTypeNameForDefaultEndpoint()
{
	static FName DefaultEndpointName = FName("Default Endpoint");
	return DefaultEndpointName;
}

FName IAudioEndpointFactory::GetModularFeatureName()
{
	static FName ModularFeatureName = "External Audio Endpoint";
	return ModularFeatureName;
}

void IAudioEndpointFactory::RegisterEndpointType(IAudioEndpointFactory* InFactory)
{
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), InFactory);
}

void IAudioEndpointFactory::UnregisterEndpointType(IAudioEndpointFactory* InFactory)
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), InFactory);
}

IAudioEndpointFactory* IAudioEndpointFactory::Get(const FName& InName)
{
	if (InName == GetTypeNameForDefaultEndpoint() || InName == FName())
	{
		return nullptr;
	}

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioEndpointFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioEndpointFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (IAudioEndpointFactory* Factory : Factories)
	{
		if (Factory && Factory->bIsImplemented && InName == Factory->GetEndpointTypeName())
		{
			return Factory;
		}
	}

	//If we got here, we're probably dealing with a platform-specific endpoint on a platform its not enabled for, so we'll want to mute it
	UE_LOGF(LogAudioEndpoints, Display, "No endpoint implementation for %ls found for this platform. Endpoint Submixes set to this type will not do anything.", *InName.ToString());
	return GetDummyFactory();
}

TArray<FName> IAudioEndpointFactory::GetAvailableEndpointTypes()
{
	// Ensure the module is loaded. This will cause any platform extension modules to load and register.
	ensure(FAudioExtensionsModule::Get() != nullptr);
	
	TArray<FName> EndpointNames;

	EndpointNames.Add(GetTypeNameForDefaultEndpoint());

	IModularFeatures::Get().LockModularFeatureList();
	TArray<IAudioEndpointFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<IAudioEndpointFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (IAudioEndpointFactory* Factory : Factories)
	{
		EndpointNames.AddUnique(Factory->GetEndpointTypeName());
	}

	return EndpointNames;
}

TUniquePtr<IAudioEndpoint> IAudioEndpointFactory::CreateNewEndpointInstance(const FAudioPluginInitializationParams& InitInfo, const IAudioEndpointSettingsProxy& InitialSettings)
{
	return TUniquePtr<IAudioEndpoint>(new IAudioEndpoint());
}

UClass* IAudioEndpointFactory::GetCustomSettingsClass() const
{
	return nullptr;
}

const UAudioEndpointSettingsBase* IAudioEndpointFactory::GetDefaultSettings() const
{
	return GetDefault<UDummyEndpointSettings>();
}

IAudioEndpointFactory* IAudioEndpointFactory::GetDummyFactory()
{
	static IAudioEndpointFactory DummyFactory = IAudioEndpointFactory();
	return &DummyFactory;
}
