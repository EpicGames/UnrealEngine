// Copyright Epic Games, Inc. All Rights Reserved.
#include "IAudioModulation.h"

#include "Containers/Map.h"
#include "UObject/Object.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IAudioModulation)


namespace Audio
{
	namespace ModulationPrivate
	{
		static std::atomic<FModulatorHandleId> NextHandleId = INDEX_NONE;

		static const FModulationParameter DefaultParameter = { };
	}

	namespace ModulationInterfacePrivate
	{
		class FModulationParameterRegistry
		{
		private:
			TMap<FName, TUniquePtr<FModulationParameter>> Values;
			TMap<FName, bool> ClampOverrides;

			mutable FCriticalSection ThreadSafeValueAccessor;
			mutable FCriticalSection ThreadSafeClampOverrideAccessor;

		public:
			bool IsRegistered(FName InName) const
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				return Values.Contains(InName);
			}

			void Register(FName InName, FModulationParameter&& InParameter)
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				if (const TUniquePtr<FModulationParameter>* ParamPtr = Values.Find(InName))
				{
					if (FModulationParameter* Param = ParamPtr->Get())
					{
						*Param = MoveTemp(InParameter);
						return;
					}
				}

				Values.Add(InName, MakeUnique<FModulationParameter>(FModulationParameter(InParameter)));
			}

			bool Unregister(FName InName)
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				return Values.Remove(InName) > 0;
			}

			void UnregisterAll()
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				Values.Reset();
			}

			void SetClampOverride(FName InName, bool bClampOverride)
			{
				FScopeLock Lock(&ThreadSafeClampOverrideAccessor);
				if (ClampOverrides.Contains(InName))
				{
					ClampOverrides[InName] = bClampOverride;
				}
				else
				{
					ClampOverrides.Add(InName, bClampOverride);
				}
			}

			void ClearClampOverride(FName InName)
			{
				FScopeLock Lock(&ThreadSafeClampOverrideAccessor);
				if (ClampOverrides.Contains(InName))
				{
					ClampOverrides.Remove(InName);
				}
			}

			void ClearAllClampOverrides()
			{
				FScopeLock Lock(&ThreadSafeClampOverrideAccessor);
				ClampOverrides.Empty();
			}
			
			const FModulationParameter* Get(FName InName) const
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				if (const TUniquePtr<FModulationParameter>* ParamPtr = Values.Find(InName))
				{
					if (const FModulationParameter* Param = ParamPtr->Get())
					{
						return Param;
					}
				}

				return nullptr;
			}

			bool GetParameterClamp(FName InName) const
			{
				FScopeLock Lock(&ThreadSafeValueAccessor);
				FScopeLock ClampLock(&ThreadSafeClampOverrideAccessor);

				// First check for overrides
				if (ClampOverrides.Contains(InName))
				{
					return ClampOverrides[InName];
				}
				
				if (const TUniquePtr<FModulationParameter>* ParamPtr = Values.Find(InName))
				{
					if (FModulationParameter* Param = ParamPtr->Get())
					{
						return Param->bClampToNormalizedRange;
					}
				}

				return true;
			}
		} ParameterRegistry;
	}

	FModulatorHandleId CreateModulatorHandleId()
	{
		return ++ModulationPrivate::NextHandleId;
	}

	FModulationParameter::FModulationParameter()
		: MixFunction(GetDefaultMixFunction())
		, UnitFunction(GetDefaultUnitConversionFunction())
		, NormalizedFunction(GetDefaultNormalizedConversionFunction())
	{
	}

	const FModulationMixFunction& FModulationParameter::GetDefaultMixFunction()
	{
		static const FModulationMixFunction DefaultMixFunction = [](float& InOutValueA, float InValueB)
		{
			InOutValueA *= InValueB;
		};

		return DefaultMixFunction;
	}

	const FModulationUnitConversionFunction& FModulationParameter::GetDefaultUnitConversionFunction()
	{
		static const FModulationUnitConversionFunction ConversionFunction = [](float& InOutValue)
		{
		};

		return ConversionFunction;
	};

	const FModulationNormalizedConversionFunction& FModulationParameter::GetDefaultNormalizedConversionFunction()
	{
		static const FModulationNormalizedConversionFunction ConversionFunction = [](float& InOutValue)
		{
		};

		return ConversionFunction;
	};

	void RegisterModulationParameter(FName InName, FModulationParameter&& InParameter)
	{
		using namespace ModulationInterfacePrivate;
		ParameterRegistry.Register(InName, MoveTemp(InParameter));
	}

	bool UnregisterModulationParameter(FName InName)
	{
		using namespace ModulationInterfacePrivate;
		return ParameterRegistry.Unregister(InName);
	}

	void UnregisterAllModulationParameters()
	{
		using namespace ModulationInterfacePrivate;
		ParameterRegistry.UnregisterAll();
	}

	bool IsModulationParameterRegistered(FName InName)
	{
		using namespace ModulationInterfacePrivate;
		return ParameterRegistry.IsRegistered(InName);
	}
	
	const FModulationParameter* GetModulationParameterPtr(FName InName)
	{
		using namespace ModulationInterfacePrivate;
		return ParameterRegistry.Get(InName);
	}

	const FModulationParameter& GetDefaultModulationParameter()
	{
		return ModulationPrivate::DefaultParameter;
	}

	void SetModulationParameterClampOverride(FName InName, bool bClamp)
	{
		ModulationInterfacePrivate::ParameterRegistry.SetClampOverride(InName, bClamp);
	}

	void ClearModulationParameterClampOverride(FName InName)
	{
		ModulationInterfacePrivate::ParameterRegistry.ClearClampOverride(InName);
	}

	void ClearAllModulationParameterClampOverrides()
	{
		ModulationInterfacePrivate::ParameterRegistry.ClearAllClampOverrides();
	}

	bool GetModulationParameterClampValue(const FName& ParameterName, bool& bClampOut)
	{
		if (ModulationInterfacePrivate::ParameterRegistry.IsRegistered(ParameterName))
		{
			bClampOut = ModulationInterfacePrivate::ParameterRegistry.GetParameterClamp(ParameterName);
			return true;
		}

		return false;
	}

	const FModulationParameter& GetModulationParameter(FName InName)
	{
		using namespace ModulationInterfacePrivate;

		if (const FModulationParameter* Param = ParameterRegistry.Get(InName))
		{
			return *Param;
		}
		
		return ModulationPrivate::DefaultParameter;
	}

	FModulatorHandle::FModulatorHandle(Audio::FModulationParameter&& InParameter)
		: Parameter(InParameter)
		, HandleId(CreateModulatorHandleId())
	{
	}

	FModulatorHandle::FModulatorHandle(IAudioModulationManager& InModulation, const Audio::IModulatorSettings& InModulatorSettings, Audio::FModulationParameter&& InParameter)
		: Parameter(MoveTemp(InParameter))
		, HandleId(CreateModulatorHandleId())
		, Modulation(InModulation.AsShared())
	{
		ModulatorTypeId = InModulatorSettings.Register(HandleId, InModulation);
		if (ModulatorTypeId != INDEX_NONE)
		{
			ModulatorId = InModulatorSettings.GetModulatorId();
		}
	}

	FModulatorHandle::FModulatorHandle(const FModulatorHandle& InOther)
	{
		HandleId = CreateModulatorHandleId();

		if (TSharedPtr<IAudioModulationManager> ModPtr = InOther.Modulation.Pin())
		{
			ModPtr->RegisterModulator(HandleId, InOther.ModulatorId);
			Parameter = InOther.Parameter;
			ModulatorId = InOther.ModulatorId;
			ModulatorTypeId = InOther.ModulatorTypeId;
			Modulation = InOther.Modulation;
		}
	}

	FModulatorHandle::FModulatorHandle(FModulatorHandle&& InOther)
		: Parameter(MoveTemp(InOther.Parameter))
		, HandleId(InOther.HandleId)
		, ModulatorTypeId(InOther.ModulatorTypeId)
		, ModulatorId(InOther.ModulatorId)
		, Modulation(InOther.Modulation)
	{
		// Move does not register as presumed already activated or
		// copying default handle, which is invalid. Removes data
		// from handle being moved to avoid double deactivation on
		// destruction.
		InOther.Parameter = FModulationParameter();
		InOther.HandleId = INDEX_NONE;
		InOther.ModulatorTypeId = INDEX_NONE;
		InOther.ModulatorId = INDEX_NONE;
		InOther.Modulation.Reset();
	}

	FModulatorHandle::~FModulatorHandle()
	{
		if (TSharedPtr<IAudioModulationManager> ModPtr = Modulation.Pin())
		{
			ModPtr->UnregisterModulator(*this);
		}
	}

	FModulatorHandle& FModulatorHandle::operator=(const FModulatorHandle& InOther)
	{
		Parameter = InOther.Parameter;

		if (TSharedPtr<IAudioModulationManager> ModPtr = InOther.Modulation.Pin())
		{
			HandleId = CreateModulatorHandleId();
			ModulatorId = InOther.ModulatorId;
			ModulatorTypeId = InOther.ModulatorTypeId;
			Modulation = InOther.Modulation;

			if (ModulatorId != INDEX_NONE)
			{
				ModPtr->RegisterModulator(HandleId, ModulatorId);
			}
		}
		else
		{
			HandleId = INDEX_NONE;
			ModulatorId = INDEX_NONE;
			ModulatorTypeId = INDEX_NONE;
			Modulation.Reset();
		}

		return *this;
	}

	FModulatorHandle& FModulatorHandle::operator=(FModulatorHandle&& InOther)
	{
		if (HandleId != INDEX_NONE)
		{
			if (TSharedPtr<IAudioModulationManager> ModPtr = Modulation.Pin())
			{
				ModPtr->UnregisterModulator(*this);
			}
		}

		// Move does not activate as presumed already activated or
		// copying default handle, which is invalid. Removes data
		// from handle being moved to avoid double deactivation on
		// destruction.
		Parameter = MoveTemp(InOther.Parameter);
		HandleId = InOther.HandleId;
		ModulatorId = InOther.ModulatorId;
		ModulatorTypeId = InOther.ModulatorTypeId;
		Modulation = InOther.Modulation;

		InOther.Parameter = FModulationParameter();
		InOther.HandleId = INDEX_NONE;
		InOther.ModulatorId = INDEX_NONE;
		InOther.ModulatorTypeId = INDEX_NONE;
		InOther.Modulation.Reset();

		return *this;
	}

	FModulatorId FModulatorHandle::GetModulatorId() const
	{
		return ModulatorId;
	}

	const FModulationParameter& FModulatorHandle::GetParameter() const
	{
		return Parameter;
	}

	FModulatorTypeId FModulatorHandle::GetTypeId() const
	{
		return ModulatorTypeId;
	}

	uint32 FModulatorHandle::GetHandleId() const
	{
		return HandleId;
	}

	bool FModulatorHandle::GetValue(float& OutValue) const
	{
		check(IsValid());

		OutValue = 1.0f;

		if (TSharedPtr<IAudioModulationManager> ModPtr = Modulation.Pin())
		{
			return ModPtr->GetModulatorValue(*this, OutValue);
		}

		return false;
	}

	bool FModulatorHandle::GetValueThreadSafe(float& OutValue) const
	{
		check(IsValid());

		OutValue = 1.0f;
		if (TSharedPtr<IAudioModulationManager> ModPtr = Modulation.Pin())
		{
			return ModPtr->GetModulatorValueThreadSafe(*this, OutValue);
		}

		return false;
	}

	bool FModulatorHandle::IsValid() const
	{
		return ModulatorId != INDEX_NONE;
	}
} // namespace Audio

const Audio::FModulationParameter& USoundModulatorBase::GetOutputParameter() const
{
	return Audio::ModulationPrivate::DefaultParameter;
}

TSharedPtr<Audio::IProxyData> USoundModulatorBase::CreateProxyData(const Audio::FProxyDataInitParams& InitParams)
{
	// This should never be hit as all instances of modulators should implement their own version of the proxy data interface.
	checkNoEntry();
	return TSharedPtr<Audio::IProxyData>();
}

TUniquePtr<Audio::IModulatorSettings> USoundModulatorBase::CreateProxySettings() const
{
	checkNoEntry();
	return TUniquePtr<Audio::IModulatorSettings>();
}

