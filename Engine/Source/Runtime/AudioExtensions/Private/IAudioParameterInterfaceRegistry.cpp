// Copyright Epic Games, Inc. All Rights Reserved.
#include "IAudioParameterInterfaceRegistry.h"

#include "Algo/Transform.h"


namespace Audio
{
	namespace ParameterInterfaceRegistryPrivate
	{
		class FParameterInterfaceRegistry : public IAudioParameterInterfaceRegistry
		{
		public:
			virtual void IterateInterfaces(TFunction<void(FParameterInterfacePtr)> InFunction) const override
			{
				for (const FParameterInterfacePtr& InterfacePtr : Interfaces)
				{
					InFunction(InterfacePtr);
				}
			}

			virtual void RegisterInterface(FParameterInterfacePtr InInterface) override
			{
				Interfaces.Add(InInterface);
				if (RegistrationFunction)
				{
					RegistrationFunction(InInterface);
				}
			}

			virtual void OnRegistration(TUniqueFunction<void(FParameterInterfacePtr)>&& InFunction) override
			{
				RegistrationFunction = MoveTemp(InFunction);
			}
		};
	} // namespace ParameterInterfaceRegistryPrivate

	FParameterInterface::FParameterInterface(
		FName InName,
		const FVersion& InVersion,
		const UClass& InType
	)
		: NamePrivate(InName)
		, VersionPrivate(InVersion)
		, UClassOptions({ { InType.GetClassPathName() } })
	{
	}

	FParameterInterface::FParameterInterface(FName InName, const FVersion& InVersion)
		: NamePrivate(InName)
		, VersionPrivate(InVersion)
	{
	}

	FName FParameterInterface::GetName() const
	{
		return NamePrivate;
	}

	const FParameterInterface::FVersion& FParameterInterface::GetVersion() const
	{
		return VersionPrivate;
	}

	const UClass& FParameterInterface::GetType() const
	{
		return *UObject::StaticClass();
	}

	const TArray<FParameterInterface::FClassOptions>& FParameterInterface::GetUClassOptions() const
	{
		return UClassOptions;
	}

	const TArray<FParameterInterface::FInput>& FParameterInterface::GetInputs() const
	{
		return Inputs;
	}

	const TArray<FParameterInterface::FOutput>& FParameterInterface::GetOutputs() const
	{
		return Outputs;
	}

	const TArray<FParameterInterface::FEnvironmentVariable>& FParameterInterface::GetEnvironment() const
	{
		return Environment;
	}

#if WITH_EDITORONLY_DATA
	const FText& FParameterInterface::GetDescription() const
	{
		return Description;
	}
#endif

TArray<const UClass*> FParameterInterface::FindSupportedUClasses() const
	{
		TArray<const UClass*> SupportedUClasses;
		for (const FClassOptions& Options : UClassOptions)
		{
			if (const UClass* Class = FindObject<const UClass>(Options.ClassPath))
			{
				SupportedUClasses.Add(Class);
			}
		}

		return SupportedUClasses;
	}

	TUniquePtr<IAudioParameterInterfaceRegistry> IAudioParameterInterfaceRegistry::Instance;

	IAudioParameterInterfaceRegistry& IAudioParameterInterfaceRegistry::Get()
	{
		using namespace ParameterInterfaceRegistryPrivate;

		if (!Instance.IsValid())
		{
			Instance = MakeUnique<FParameterInterfaceRegistry>();
		}
		return *Instance;
	}
} // namespace Audio
