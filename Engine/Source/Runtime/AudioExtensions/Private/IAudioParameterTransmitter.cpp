// Copyright Epic Games, Inc. All Rights Reserved.
#include "IAudioParameterTransmitter.h"
#include "UObject/Object.h"


namespace Audio
{
	const FName IParameterTransmitter::RouterName = "ParameterTransmitter";

	TArray<const TObjectPtr<UObject>*> ILegacyParameterTransmitter::GetReferencedObjects() const
	{
		return { };
	}

	void ILegacyParameterTransmitter::AddReferencedObjects(FReferenceCollector& InCollector)
	{
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
	FParameterTransmitterBase::FParameterTransmitterBase(TArray<FAudioParameter> InDefaultParams)
		: AudioParameters(MoveTemp(InDefaultParams))
		, bIsVirtualized(false)
	{
	}

	FParameterTransmitterBase::~FParameterTransmitterBase() = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FParameterTransmitterBase::GetParameter(FName InName, FAudioParameter& OutValue) const
	{
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		if (const FAudioParameter* Param = FAudioParameter::FindParam(AudioParameters, InName))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			OutValue = *Param;
			return true;
		}

		return false;
	}

	void FParameterTransmitterBase::ResetParameters() 
	{
		check(IsInAudioThread());

PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		AudioParameters.Reset();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	const TArray<FAudioParameter>& FParameterTransmitterBase::GetParameters() const
	{
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		return AudioParameters;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FParameterTransmitterBase::CopyParameters(TArray<FAudioParameter>& OutParameters) const
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		OutParameters = AudioParameters;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool FParameterTransmitterBase::SetParameters(TArray<FAudioParameter>&& InParameters)
	{
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		FAudioParameter::Merge(MoveTemp(InParameters), AudioParameters);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return true;
	}

	void FParameterTransmitterBase::AddReferencedObjects(FReferenceCollector& InCollector)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because AudioParameters will become private
		AddReferencedObjectsFromParameters(InCollector, AudioParameters);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FParameterTransmitterBase::OnVirtualizeActiveSound()
	{ 
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because bIsVirtualized will become private
		bIsVirtualized = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	 void FParameterTransmitterBase::OnRealizeVirtualizedActiveSound(TArray<FAudioParameter>&& InParameters)
	 { 
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because bIsVirtualized will become private
		bIsVirtualized = false;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		SetParameters(MoveTemp(InParameters));
	 }

	 bool FParameterTransmitterBase::IsVirtualized() const
	 {
		check(IsInAudioThread());
PRAGMA_DISABLE_DEPRECATION_WARNINGS // deprecation is because bIsVirtualized will become private
		return bIsVirtualized;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	 }

	void FParameterTransmitterBase::AddReferencedObjectsFromParameters(FReferenceCollector& InCollector, TArrayView<FAudioParameter> InParameters) const
	{
		for (FAudioParameter& Param : InParameters)
		{
			if (Param.ObjectParam)
			{
				InCollector.AddReferencedObject(Param.ObjectParam);
			}

			for (TObjectPtr<UObject>& Object : Param.ArrayObjectParam)
			{
				if (Object)
				{
					InCollector.AddReferencedObject(Object);
				}
			}
		}
	}
} // namespace Audio
