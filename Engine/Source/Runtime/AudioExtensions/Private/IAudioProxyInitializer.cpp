// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAudioProxyInitializer.h"

#include "AudioExtensionsLog.h"
#include "Logging/LogMacros.h"
#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

TUniquePtr<Audio::IProxyData> IAudioProxyDataFactory::CreateNewProxyData(const Audio::FProxyDataInitParams& InitParams) 
{
	return nullptr;
}

TSharedPtr<Audio::IProxyData> IAudioProxyDataFactory::CreateProxyData(const Audio::FProxyDataInitParams& InitParams) 
{
	// If the new interface is not overridden, use deprecated interface
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TUniquePtr<Audio::IProxyData> Proxy = CreateNewProxyData(InitParams);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Only log an error once per a proxy data type to avoid logspam.
	static TSet<FName> LoggedProxyWarnings;
	if (Proxy.IsValid())
	{
		const FName ProxyTypeName = Proxy->GetProxyTypeName();
		if (!LoggedProxyWarnings.Contains(ProxyTypeName))
		{
			LoggedProxyWarnings.Add(ProxyTypeName);
			UE_LOGF(LogAudioExtensions, Warning, "Use of deprecated 'TUniquePtr<Audio::IProxyData> CreateNewProxyData(...)' for proxy type \"%ls\". Please override 'TSharedPtr<Audio::IProxyData> CreateProxyData(...)'", *ProxyTypeName.ToString());
		}
	}

	// Pass ownership of proxy from unique ptr to shared ptr. 
	return TSharedPtr<Audio::IProxyData>(Proxy.Release());
}
