// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISoundfieldEndpoint.h"

#include "AudioExtentionsModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISoundfieldEndpoint)

ISoundfieldEndpoint::ISoundfieldEndpoint(int32 NumRenderCallbacksToBuffer)
{
	NumRenderCallbacksToBuffer = FMath::Max(NumRenderCallbacksToBuffer, 2);
	AudioPacketBuffer.AddDefaulted(NumRenderCallbacksToBuffer);
}

bool ISoundfieldEndpoint::PushAudio(TUniquePtr<ISoundfieldAudioPacket>&& InPacket)
{
	if (GetRemainderInPacketBuffer() > 0)
	{
		int32 WriteIndex = WriteCounter.GetValue();
		AudioPacketBuffer[WriteIndex] = MoveTemp(InPacket);
		WriteCounter.Set((WriteIndex + 1) % AudioPacketBuffer.Num());
		return true;
	}
	else
	{
		return false;
	}
}

void ISoundfieldEndpoint::SetNewSettings(TUniquePtr<ISoundfieldEndpointSettingsProxy>&& InNewSettings)
{
	FScopeLock ScopeLock(&CurrentSettingsCriticalSection);
	CurrentSettings = MoveTemp(InNewSettings);
}

int32 ISoundfieldEndpoint::GetNumPacketsBuffer()
{
	const int32 ReadIndex = ReadCounter.GetValue();
	const int32 WriteIndex = WriteCounter.GetValue();

	if (WriteIndex >= ReadIndex)
	{
		return WriteIndex - ReadIndex;
	}
	else
	{
		return AudioPacketBuffer.Num() - ReadIndex + WriteIndex;
	}
}

int32 ISoundfieldEndpoint::GetRemainderInPacketBuffer()
{
	const uint32 ReadIndex = ReadCounter.GetValue();
	const uint32 WriteIndex = WriteCounter.GetValue();

	return (AudioPacketBuffer.Num() - 1 - WriteIndex + ReadIndex) % AudioPacketBuffer.Num();
}

void ISoundfieldEndpoint::ProcessAudioIfNecessary()
{
	const bool bShouldProcessAudio = !RenderCallback.IsValid() && EndpointRequiresCallback();
	if (bShouldProcessAudio)
	{
		RunCallbackSynchronously();
	}
}

TUniquePtr<ISoundfieldAudioPacket> ISoundfieldEndpoint::PopAudio()
{
	if (GetNumPacketsBuffer() > 0)
	{
		const int32 ReadIndex = ReadCounter.GetValue();
		TUniquePtr<ISoundfieldAudioPacket> PoppedPacket = MoveTemp(AudioPacketBuffer[ReadIndex]);
		ReadCounter.Set((ReadIndex + 1) % AudioPacketBuffer.Num());
		return PoppedPacket;
	}
	else
	{
		return nullptr;
	}
}

void ISoundfieldEndpoint::PollSettings(TFunctionRef<void(const ISoundfieldEndpointSettingsProxy*)> SettingsCallback)
{
	FScopeLock ScopeLock(&CurrentSettingsCriticalSection);
	SettingsCallback(CurrentSettings.Get());
}

void ISoundfieldEndpoint::StartRunningCallback()
{
	float CallbackDuration = GetDesiredCallbackPeriodicity();

	RenderCallback.Reset(new Audio::FMixerNullCallback(CallbackDuration, [&]()
	{
		RunCallbackSynchronously();
	}));
}

void ISoundfieldEndpoint::StopRunningCallback()
{
	RenderCallback.Reset();
}

void ISoundfieldEndpoint::RunCallbackSynchronously()
{
	auto CallbackWithSettings = [&](const ISoundfieldEndpointSettingsProxy* InSettings)
	{
		while (TUniquePtr<ISoundfieldAudioPacket> PoppedPacket = PopAudio())
		{
			OnAudioCallback(MoveTemp(PoppedPacket), InSettings);
		}
	};

	PollSettings(CallbackWithSettings);
}

ISoundfieldEndpointFactory* ISoundfieldEndpointFactory::Get(const FName& InName)
{
	if (InName == DefaultSoundfieldEndpointName() || InName == FName())
	{
		return nullptr;
	}
	IModularFeatures::Get().LockModularFeatureList();
	TArray<ISoundfieldEndpointFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<ISoundfieldEndpointFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ISoundfieldEndpointFactory* Factory : Factories)
	{
		if (Factory && InName == Factory->GetEndpointTypeName())
		{
			return Factory;
		}
	}

	ensureMsgf(false, TEXT("Soundfield Endpoint Type %s not found!"), *InName.ToString());
	return nullptr;
}

TArray<FName> ISoundfieldEndpointFactory::GetAllSoundfieldEndpointTypes()
{
	// Ensure the module is loaded. This will cause any platform extension modules to load and register. 
	ensure(FAudioExtensionsModule::Get() != nullptr);
	
	TArray<FName> SoundfieldFormatNames;

	SoundfieldFormatNames.Add(DefaultSoundfieldEndpointName());

	IModularFeatures::Get().LockModularFeatureList();
	TArray<ISoundfieldEndpointFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<ISoundfieldEndpointFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ISoundfieldEndpointFactory* Factory : Factories)
	{
		SoundfieldFormatNames.Add(Factory->GetEndpointTypeName());
	}

	return SoundfieldFormatNames;
}

FName ISoundfieldEndpointFactory::DefaultSoundfieldEndpointName()
{
	static FName DefaultEndpointName = FName("Default Soundfield Endpoint");
	return DefaultEndpointName;
}

TUniquePtr<ISoundfieldDecoderStream> ISoundfieldEndpointFactory::CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings)
{
	// Endpoint soundfield formats don't ever get decoded in our audio engine, since they are external sends.
	checkNoEntry();
	return nullptr;
}

FName ISoundfieldEndpointFactory::GetSoundfieldFormatName()
{
	return GetEndpointTypeName();
}

bool ISoundfieldEndpointFactory::CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings)
{
	return false;
}

