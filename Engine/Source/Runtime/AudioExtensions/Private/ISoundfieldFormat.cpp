// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISoundfieldFormat.h"

#include "AudioExtentionsModule.h"
#include "ISoundfieldEndpoint.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISoundfieldFormat)

FName ISoundfieldFactory::GetFormatNameForNoEncoding()
{
	static FName NoEncodingFormatName = FName("No Encoding");
	return NoEncodingFormatName;
}

FName ISoundfieldFactory::GetFormatNameForInheritedEncoding()
{
	static FName InheritedFormatName = FName("Inherited Encoding");
	return InheritedFormatName;
}

FName ISoundfieldFactory::GetModularFeatureName()
{
	static FName SoundfieldFactoryName = FName("Soundfield Format");
	return SoundfieldFactoryName;
}

void ISoundfieldFactory::RegisterSoundfieldFormat(ISoundfieldFactory* InFactory)
{
	check(IsInGameThread());
	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), InFactory);
}

void ISoundfieldFactory::UnregisterSoundfieldFormat(ISoundfieldFactory* InFactory)
{
	check(IsInGameThread());
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), InFactory);
}

ISoundfieldFactory* ISoundfieldFactory::Get(const FName& InName)
{
	if (InName == GetFormatNameForNoEncoding() || InName == FName())
	{
		return nullptr;
	}

	IModularFeatures::Get().LockModularFeatureList();
	TArray<ISoundfieldFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<ISoundfieldFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ISoundfieldFactory* Factory : Factories)
	{
		if (Factory && InName == Factory->GetSoundfieldFormatName())
		{
			if (Factory->IsEndpointFormat())
			{
				ensureAlwaysMsgf(false, TEXT("This format is only supported for endpoints. Use ISoundfieldEndpointFactory::Get instead."));
			}

			return Factory;
		}
	}

	ensureAlwaysMsgf(false, TEXT("Soundfield Format %s not found!"), *InName.ToString());
	return nullptr;
}

TArray<FName> ISoundfieldFactory::GetAvailableSoundfieldFormats()
{
	// Ensure the module is loaded. This will cause any platform extension modules to load and register. 
	ensure(FAudioExtensionsModule::Get() != nullptr);
	
	TArray<FName> SoundfieldFormatNames;

	SoundfieldFormatNames.Add(GetFormatNameForInheritedEncoding());
	SoundfieldFormatNames.Add(GetFormatNameForNoEncoding());

	IModularFeatures::Get().LockModularFeatureList();
	TArray<ISoundfieldFactory*> Factories = IModularFeatures::Get().GetModularFeatureImplementations<ISoundfieldFactory>(GetModularFeatureName());
	IModularFeatures::Get().UnlockModularFeatureList();

	for (ISoundfieldFactory* Factory : Factories)
	{
		SoundfieldFormatNames.Add(Factory->GetSoundfieldFormatName());
	}

	return SoundfieldFormatNames;
}

