// Copyright Epic Games, Inc. All Rights Reserved.


#include "IWaveformTransformation.h"
#include "Templates/SharedPointer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IWaveformTransformation)

TArray<Audio::FTransformationPtr> UWaveformTransformationChain::CreateTransformations() const
{
	TArray<Audio::FTransformationPtr> TransformationPtrs;

	for(UWaveformTransformationBase* Transformation : Transformations)
	{
		if(Transformation)
		{
			TransformationPtrs.Add(Transformation->CreateTransformation());
		}
	}
	
	return TransformationPtrs;
}

#if WITH_EDITOR
void UWaveformTransformationBase::PostEditUndo()
{
	Super::PostEditUndo();

	constexpr bool bMarkFileDirty = true;
	OnTransformationChanged.ExecuteIfBound(bMarkFileDirty);
}

void UWaveformTransformationBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const bool bMarkFileDirty = (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive);
	OnTransformationChanged.ExecuteIfBound(bMarkFileDirty);
}

void UWaveformTransformationBase::NotifyPropertyChange(FProperty* Property)
{
	if (Property)
	{
		FPropertyChangedEvent PropertyChangedEvent(Property);
		PostEditChangeProperty(PropertyChangedEvent);
	}
}
#endif //WITH_EDITOR

int64 FWaveTransformUObjectConfiguration::GetEndFrameOffset() const
{
	if (EndTime < 0.f || OriginalNumFrames <= 0 || SampleRate <= 0.f)
	{
		return 0;
	}

	int64 EndTimeInFrames = FMath::RoundToInt64(EndTime * SampleRate);
	
	// Clamp to OriginalNumFrames to prevent a negative offset
	return FMath::Clamp((OriginalNumFrames - EndTimeInFrames), 0, OriginalNumFrames);
}
