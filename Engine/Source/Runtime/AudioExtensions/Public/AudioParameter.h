// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IAudioProxyInitializer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "AudioParameter.generated.h"

class UObject;


#define AUDIO_PARAMETER_NAMESPACE_PATH_DELIMITER "."

// Convenience macro for statically declaring an interface member's FName. AUDIO_PARAMETER_INTERFACE_NAMESPACE must be defined.
#define AUDIO_PARAMETER_INTERFACE_MEMBER_DEFINE(Name) (AUDIO_PARAMETER_INTERFACE_NAMESPACE AUDIO_PARAMETER_NAMESPACE_PATH_DELIMITER Name)

namespace Audio
{
	struct FParameterPath
	{
		static AUDIOEXTENSIONS_API const FString NamespaceDelimiter;

		// Combines names using the namespace delimiter
		static AUDIOEXTENSIONS_API FName CombineNames(FName InLeft, FName InRight);

		// Splits name into namespace & parameter name
		static AUDIOEXTENSIONS_API void SplitName(FName InFullName, FName& OutNamespace, FName& OutParameterName);
	};
}

UENUM()
enum class EAudioParameterType : uint8
{
	// 'Default' results in behavior that is resolved
	// based on the system interpreting it.  To support
	// legacy implementation, SoundCues cache all typed values
	// associated with a given parameter name. 
	// For MetaSounds, use a specific Type instead of this one. 
	None UMETA(DisplayName = "Default"),

	// Boolean value
	Boolean,

	// Integer value
	Integer,

	// Float value
	Float,

	// String value (not supported by legacy SoundCue system)
	String,

	// Object value (types other than SoundWave not supported by legacy SoundCue system)
	Object,

	// Array of default initialized values (not supported by legacy SoundCue system)
	// Hidden for now as no parameter types exist that support default construction
	NoneArray UMETA(Hidden, DisplayName = "Default (Array)"),

	// Array of boolean values (not supported by legacy SoundCue system)
	BooleanArray UMETA(DisplayName = "Boolean (Array)"),

	// Array of integer values (not supported by legacy SoundCue system)
	IntegerArray UMETA(DisplayName = "Integer (Array)"),

	// Array of float values (not supported by legacy SoundCue system)
	FloatArray UMETA(DisplayName = "Float (Array)"),

	// Array of string values (not supported by legacy SoundCue system)
	StringArray UMETA(DisplayName = "String (Array)"),

	// Array of object values (not supported by legacy SoundCue system)
	ObjectArray UMETA(DisplayName = "Object (Array)"),

	// Trigger value
	Trigger,

	COUNT UMETA(Hidden)
};


USTRUCT(BlueprintType)
struct FAudioParameter
{
	GENERATED_USTRUCT_BODY()

	FAudioParameter() = default;

	FAudioParameter(FName InName)
		: ParamName(InName)
	{
	}

	FAudioParameter(FName InName, float InValue)
		: ParamName(InName)
		, FloatParam(InValue)
		, ParamType(EAudioParameterType::Float)
	{
	}

	FAudioParameter(FName InName, bool InValue)
		: ParamName(InName)
		, BoolParam(InValue)
		, ParamType(EAudioParameterType::Boolean)
	{
	}

	FAudioParameter(FName InName, int32 InValue)
		: ParamName(InName)
		, IntParam(InValue)
		, ParamType(EAudioParameterType::Integer)
	{
	}

	FAudioParameter(FName InName, UObject* InValue)
		: ParamName(InName)
		, ObjectParam(InValue)
		, ParamType(EAudioParameterType::Object)
	{
	}

	FAudioParameter(FName InName, const FString& InValue)
		: ParamName(InName)
		, StringParam(InValue)
		, ParamType(EAudioParameterType::String)
	{
	}

	FAudioParameter(FName InName, const TArray<float>& InValue)
		: ParamName(InName)
		, ArrayFloatParam(InValue)
		, ParamType(EAudioParameterType::FloatArray)
	{
	}

	FAudioParameter(FName InName, TArray<float>&& InValue)
		: ParamName(InName)
		, ArrayFloatParam(MoveTemp(InValue))
		, ParamType(EAudioParameterType::FloatArray)
	{
	}

	FAudioParameter(FName InName, const TArray<bool>& InValue)
		: ParamName(InName)
		, ArrayBoolParam(InValue)
		, ParamType(EAudioParameterType::BooleanArray)
	{
	}

	FAudioParameter(FName InName, TArray<bool>&& InValue)
		: ParamName(InName)
		, ArrayBoolParam(MoveTemp(InValue))
		, ParamType(EAudioParameterType::BooleanArray)
	{
	}

	FAudioParameter(FName InName, const TArray<int32>& InValue)
		: ParamName(InName)
		, ArrayIntParam(InValue)
		, ParamType(EAudioParameterType::IntegerArray)
	{
	}

	FAudioParameter(FName InName, TArray<int32>&& InValue)
		: ParamName(InName)
		, ArrayIntParam(MoveTemp(InValue))
		, ParamType(EAudioParameterType::IntegerArray)
	{
	}

	FAudioParameter(FName InName, const TArray<UObject*>& InValue)
		: ParamName(InName)
		, ArrayObjectParam(InValue)
		, ParamType(EAudioParameterType::ObjectArray)
	{
	}

// We can't move TArray<UObject*> into a TArray<TObjectPtr<UObject>> since UE_OBJECT_PTR_GC_BARRIER was added
#if 0
	FAudioParameter(FName InName, TArray<UObject*>&& InValue)
		: ParamName(InName)
		, ArrayObjectParam(MoveTemp(InValue))
		, ParamType(EAudioParameterType::ObjectArray)
	{
	}
#endif

	FAudioParameter(FName InName, const TArray<FString>& InValue)
		: ParamName(InName)
		, ArrayStringParam(InValue)
		, ParamType(EAudioParameterType::StringArray)
	{
	}

	FAudioParameter(FName InName, TArray<FString>&& InValue)
		: ParamName(InName)
		, ArrayStringParam(MoveTemp(InValue))
		, ParamType(EAudioParameterType::StringArray)
	{
	}

	FAudioParameter(FName InName, EAudioParameterType Type)
		: ParamName(InName)
		, ParamType(Type)
	{
		if (Type == EAudioParameterType::Trigger)
		{
			BoolParam = true;
		}
	}

	// Static function to avoid int32 constructor collision
	static FAudioParameter CreateDefaultArray(FName InName, int32 InNum)
	{
		FAudioParameter NewParam(InName, InNum);
		NewParam.ParamType = EAudioParameterType::NoneArray;
		return NewParam;
	}

	// Sets values specified by type field of the given parameter on this parameter. If the provided parameter is set to type 'None', takes all values of the given parameter.
	// bInTakeName = Take the name of the provided parameter.
	// bInTakeType = Take the type of the provided parameter.
	// bInMergeArrayTypes - Appends array(s) for specified type if true, else swaps the local value with that of the provided parameter if false.
	AUDIOEXTENSIONS_API void Merge(const FAudioParameter& InParameter, bool bInTakeName = true, bool bInTakeType = true, bool bInMergeArrayTypes = false);

	// Moves InParams to OutParams that are not already included. For backward compatibility (i.e. SoundCues),
	// if a param is already in OutParams, attempts to merge param values together, but assigns the param the
	// incoming param's type. Currently existing OutParam values left if new value of the same type is provided
	// by InParam.
	static AUDIOEXTENSIONS_API void Merge(TArray<FAudioParameter>&& InParams, TArray<FAudioParameter>& OutParams);

	FAudioParameter(FAudioParameter&& InParameter) = default;
	FAudioParameter(const FAudioParameter& InParameter) = default;

	FAudioParameter& operator=(const FAudioParameter& InParameter) = default;
	FAudioParameter& operator=(FAudioParameter&& InParameter) = default;

	// Name of the parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName="Name"), Category = AudioParameter)
	FName ParamName;

	// Float value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Float)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::None || ParamType == EAudioParameterType::Float"), Category = AudioParameter)
	float FloatParam = 0.f;

	// Boolean value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Bool)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::None || ParamType == EAudioParameterType::Boolean"), Category = AudioParameter)
	bool BoolParam = false;

	// Integer value of parameter. If set to 'Default Construct', value is number of array items to construct.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Int)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::None || ParamType == EAudioParameterType::Integer || ParamType == EAudioParameterType::NoneArray"), Category = AudioParameter)
	int32 IntParam = 0;

	// Object value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Object)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::None || ParamType == EAudioParameterType::Object"), Category = AudioParameter)
	TObjectPtr<UObject> ObjectParam = nullptr;

	// String value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (String)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::String"), Category = AudioParameter)
	FString StringParam;

	// Array Float value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Float Array)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::FloatArray"), Category = AudioParameter)
	TArray<float> ArrayFloatParam;

	// Boolean value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Bool Array)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::BooleanArray"), Category = AudioParameter)
	TArray<bool> ArrayBoolParam;

	// Integer value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Int Array)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::IntegerArray"), Category = AudioParameter)
	TArray<int32> ArrayIntParam;

	// Object value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (Object Array)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::ObjectArray"), Category = AudioParameter)
	TArray<TObjectPtr<UObject>> ArrayObjectParam;

	// String value of parameter
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Value (String Array)", DisplayAfter = "ParamType", EditConditionHides, EditCondition = "ParamType == EAudioParameterType::StringArray"), Category = AudioParameter)
	TArray<FString> ArrayStringParam;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (DisplayName = "Type"), Category = AudioParameter)
	EAudioParameterType ParamType = EAudioParameterType::None;

	// Optional TypeName used to describe what constructed type this parameter should be initializing.
	UPROPERTY()
	FName TypeName;

	// Object proxies to be generated when parameter is passed to the AudioThread to represent ObjectParam/ArrayObjectParam safely
	TArray<TSharedPtr<Audio::IProxyData>> ObjectProxies;

	// Common find algorithm for default/legacy parameter system
	static const FAudioParameter* FindParam(const TArray<FAudioParameter>& InParams, FName InParamName)
	{
		if (!InParamName.IsNone())
		{
			for (const FAudioParameter& ExistingParam : InParams)
			{
				if (ExistingParam.ParamName == InParamName)
				{
					return &ExistingParam;
				}
			}
		}

		return nullptr;
	}

	// Common find & add algorithm for default/legacy parameter system.
	static FAudioParameter* FindOrAddParam(TArray<FAudioParameter>& OutParams, FName InParamName)
	{
		FAudioParameter* Param = nullptr;
		if (InParamName.IsNone())
		{
			return Param;
		}

		for (FAudioParameter& ExistingParam : OutParams)
		{
			if (ExistingParam.ParamName == InParamName)
			{
				Param = &ExistingParam;
				break;
			}
		}

		if (!Param)
		{
			Param = &OutParams.AddDefaulted_GetRef();
			Param->ParamName = InParamName;
		}

		return Param;
	}
};
