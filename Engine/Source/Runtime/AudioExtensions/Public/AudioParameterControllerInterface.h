// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioParameter.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "IAudioProxyInitializer.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Interface.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "AudioParameterControllerInterface.generated.h"

class UObject;
struct FAudioParameter;
struct FFrame;


UINTERFACE(BlueprintType, meta = (CannotImplementInterfaceInBlueprint), MinimalAPI)
class UAudioParameterControllerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

// Base interface for any object implementing parameter control for a given sound instance controller.
class IAudioParameterControllerInterface : public IInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	// Resets all parameters to their original values.
	UFUNCTION(BlueprintCallable, Category = "Audio|Parameter")
	virtual void ResetParameters() = 0;

	// Executes a named trigger.  Does *not* cache trigger value, so only executes if the sound
	// is already playing.  If the intent is for the trigger to execute immediately (if playing)
	// and be called on initialization for all future instances, call 'SetBoolParameter' with the
	// intended initial trigger behavior (true if trigger desired on initialization, false if not).
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Execute Trigger Parameter"), Category = "Audio|Parameter")
	virtual void SetTriggerParameter(FName InName) = 0;

	// Sets a named Boolean
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Boolean Parameter"), Category = "Audio|Parameter")
	virtual void SetBoolParameter(FName InName, bool InBool) = 0;
	
	// Sets a named Boolean Array
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Boolean Array Parameter"), Category = "Audio|Parameter")
	virtual void SetBoolArrayParameter(FName InName, const TArray<bool>& InValue) = 0;

	// Sets a named Int32
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Integer Parameter"), Category = "Audio|Parameter")
	virtual void SetIntParameter(FName InName, int32 InInt) = 0;

	// Sets a named Int32 Array
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Integer Array Parameter"), Category = "Audio|Parameter")
	virtual void SetIntArrayParameter(FName InName, const TArray<int32>& InValue) = 0;

	// Sets a named Float
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Float Parameter"), Category = "Audio|Parameter")
	virtual void SetFloatParameter(FName InName, float InFloat) = 0;
	
	// Sets a named Float Array
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Float Array Parameter"), Category = "Audio|Parameter")
	virtual void SetFloatArrayParameter(FName InName, const TArray<float>& InValue) = 0;
	
	// Sets a named String
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set String Parameter"), Category = "Audio|Parameter")
	virtual void SetStringParameter(FName InName, const FString& InValue) = 0;

	// Sets a named String Array
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set String Array Parameter"), Category = "Audio|Parameter")
	virtual void SetStringArrayParameter(FName InName, const TArray<FString>& InValue) = 0;

	// Sets a named UObject
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Object Parameter"), Category = "Audio|Parameter")
	virtual void SetObjectParameter(FName InName, UObject* InValue) = 0;

	// Sets a named UObject Array
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Object Array Parameter"), Category = "Audio|Parameter")
	virtual void SetObjectArrayParameter(FName InName, const TArray<UObject*>& InValue) = 0;

	// Sets an array of parameters as a batch
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Set Parameters"), Category = "Audio|Parameter")
	virtual void SetParameters_Blueprint(const TArray<FAudioParameter>& InParameters) = 0;

	// Sets a named parameter to the given parameter structure value
	virtual void SetParameter(FAudioParameter&& InValue) = 0;

	// Sets an array of parameters as a batch
	virtual void SetParameters(TArray<FAudioParameter>&& InValues) = 0;

	// Template Specialization for non-script clients.
	template<typename DataType> void SetParameter(FName InName, DataType&&) = delete;
	template<> void SetParameter(FName InName, bool&& InBool) { SetParameter({ InName, MoveTemp(InBool) }); }
	template<> void SetParameter(FName InName, float&& InFloat) { SetParameter({ InName, MoveTemp(InFloat) }); }
	template<> void SetParameter(FName InName, int32&& InInteger) { SetParameter({ InName, MoveTemp(InInteger) }); }
	template<> void SetParameter(FName InName, FString&& InString) { SetParameter({ InName, MoveTemp(InString) }); }
	template<> void SetParameter(FName InName, UObject*&& InObject) { SetParameter({ InName, MoveTemp(InObject) }); }
	template<> void SetParameter(FName InName, TArray<bool>&& InBools) { SetParameter({ InName, MoveTemp(InBools)}); }
	template<> void SetParameter(FName InName, TArray<float>&& InFloats) { SetParameter({ InName, MoveTemp(InFloats) }); }
	template<> void SetParameter(FName InName, TArray<int32>&& InIntegers) { SetParameter({ InName, MoveTemp(InIntegers) }); }
	template<> void SetParameter(FName InName, TArray<FString>&& InStrings) { SetParameter({ InName, MoveTemp(InStrings) }); }
	template<> void SetParameter(FName InName, TArray<UObject*>&& InObjects) { SetParameter({ InName, MoveTemp(InObjects) }); }
};
