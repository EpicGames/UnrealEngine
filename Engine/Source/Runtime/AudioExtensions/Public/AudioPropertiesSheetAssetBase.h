// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "UObject/Interface.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "AudioPropertiesSheetAssetBase.generated.h"

UINTERFACE(MinimalApi, meta=(CannotImplementInterfaceInBlueprint))
class UAudioPropertiesSheetAssetUserInterface : public UInterface
{
	GENERATED_BODY()
};

/**
*	IAudioPropertiesSheetAssetUserInterface
* 
*	Target objects can implement this interface to bypass parsing coming from a property sheet. 
*
*	Allow/Ignore Property Parsing can be used to set a local property bypass - e.g. from a details view.
*
*	ShouldParseProperty can be used by a parser to determine if a property should be parsed or not.
*/
class IAudioPropertiesSheetAssetUserInterface
{
	GENERATED_BODY()

public:
#if WITH_EDITORONLY_DATA
	virtual void AllowPropertyParsing(const FProperty& TargetProperty) = 0;
	virtual void IgnorePropertyParsing(const FProperty& PropertyToIgnore) = 0;
	virtual bool ShouldParseProperty(const FProperty& TargetProperty) const = 0;
#endif
};

UCLASS(MinimalAPI, Abstract)
class UAudioPropertiesSheetAssetBase : public UObject
{
	GENERATED_BODY()

public: 
#if WITH_EDITOR
	UFUNCTION(BlueprintCallable, BlueprintPure=false, Category="AudioProperties")
	virtual bool CopyToObjectProperties(UObject* TargetObject) const { return false; };

	virtual FDelegateHandle BindPropertiesCopyToSheetChanges(UObject* TargetObject) { return FDelegateHandle(); };
	virtual void UnbindCopyFromPropertySheetChanges(UObject* ObjectToUnbind) {};
#endif
};
