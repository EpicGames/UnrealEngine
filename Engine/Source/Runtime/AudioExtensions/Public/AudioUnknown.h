// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"
#include "Concepts/ConvertibleTo.h"

namespace Audio
{
	/**
	 * IUnknown like interface for querying unknown data objects of their interfaces.
	 * https://en.wikipedia.org/wiki/IUnknown
	 */
	struct IUnknown
	{
		virtual ~IUnknown() = default;
		virtual const void* QueryInterface(const FName InInterfaceName) const
		{
			return const_cast<IUnknown*>(this)->QueryInterface(InInterfaceName);
		}
		virtual void* QueryInterface(const FName InInterfaceName) = 0;
	};
	
	template<typename T>
	concept CHasInterfaceIdFunction = requires 
	{
		{ T::GetInterfaceId() } -> UE::CConvertibleTo<FName>;
	};
			
	template <CHasInterfaceIdFunction TInterface>
	const TInterface* QueryInterface(const IUnknown* InUnknown)
	{
		if (InUnknown)
		{
			return static_cast<const TInterface*>(InUnknown->QueryInterface(TInterface::GetInterfaceId()));
		}
		return nullptr;
	}

	template <CHasInterfaceIdFunction TInterface>
	TInterface* QueryInterface(IUnknown* InUnknown)
	{
		if (InUnknown)
		{
			return static_cast<TInterface*>(InUnknown->QueryInterface(TInterface::GetInterfaceId()));
		}
		return nullptr;
	}
}