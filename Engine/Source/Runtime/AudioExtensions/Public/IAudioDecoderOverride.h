// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

class ICompressedAudioInfo;

/**
 * Allows you to define a factory that can provide an alternative audio file decoder at runtime.
 */
class UE_EXPERIMENTAL(5.8, "Decoder overrides are experimental and the API may be altered or removed") IAudioDecoderOverride
{
public:
	/** Returns an overriden decoder for this sound, or nullptr to allow the default decoder to be created. */
	virtual ICompressedAudioInfo* CreateOverrideDecoder(const FName SoundAssetName) = 0;

	/** Test if an overriden decoder will be created for this sound. */
	virtual bool WantToOverrideDecoder(const FName SoundAssetName) = 0;

protected:
	virtual ~IAudioDecoderOverride() = default;
};
