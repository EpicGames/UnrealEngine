// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerNullDevice.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "DSP/MultithreadedPatching.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "HAL/CriticalSection.h"
#include "ISoundfieldFormat.h"
#include "Logging/LogMacros.h"
#include "Misc/AssertionMacros.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IAudioEndpoint.generated.h"

class UClass;
struct FAudioPluginInitializationParams;
template <typename FuncType> class TFunctionRef;

DECLARE_LOG_CATEGORY_EXTERN(LogAudioEndpoints, Display, All);

/**
 * Interfaces for Endpoints
 * 
 * This set of interfaces is useful for push audio to arbitrary outputs.
 * 
 */

/**  
 * This interface should be used to provide a non-uclass version of the data described in
 * your implementation of UAudioEndpointSettingsBase.
 */
class IAudioEndpointSettingsProxy
{
public:
	virtual ~IAudioEndpointSettingsProxy() = default;
};

/**
 * This opaque class should be used for specifying settings for how audio should be
 * send to an external endpoint.
 */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class UAudioEndpointSettingsBase : public UObject
{
	GENERATED_BODY()

public:
	virtual TUniquePtr<IAudioEndpointSettingsProxy> GetProxy() const PURE_VIRTUAL(UAudioEndpointSettingsBase::GetProxy, return nullptr;);
};

//A blank class for when unimplemented endpoint types are returned
class FDummyEndpointSettingsProxy : public IAudioEndpointSettingsProxy
{
};

UCLASS()
class UDummyEndpointSettings : public UAudioEndpointSettingsBase
{
	GENERATED_BODY()

	public:
		virtual TUniquePtr<IAudioEndpointSettingsProxy> GetProxy() const override;
};

/**
 * Class that allows audio to be sent to an arbitrary locale. This can be used for multi-device rendering, haptics systems, etc. 
 * Note that this only for interleaved audio buffers with no metadata for object-based or soundfield-based rendering.
 * For those, see 
 */
class IAudioEndpoint
{
public:
	virtual ~IAudioEndpoint() = default;

	/**
	 * Create a new patch point for this endpoint. Please see MultithreadedPatching.h to see how to use the FPatchInput class.
	 * @param [in] ExpectedDurationPerRender The worst-case amount of time expected between render callbacks. This is used to determined how much to allocate for this input.
	 * @param [out] OutSampleRate. The sample rate expected to be pushed to the returned FPatchInput.
	 *                             If you need to resample your input, please see Audio::FResampler.
	 * @param [out] OutNumChannels. The number of channels expected to be pushed to the returned FPatchInput.
	 *                              If you need to downmix or upmix your input, try using the following:
	 *                              TArray<float> MixdownGainsMap;
	 *                              Audio::FMixerDevice::Get2DChannelMap(false, NumInputChannels, NumOutputChannels, false, MixdownGainsMap);
	 *                             	Audio::DownmixBuffer(NumInputChannels, NumOutputChannels, InputAudio, OutputAudio, MixdownGainsMap.GetData());
	 * @returns a new Audio::FPatchInput. The FPatchInput may be disconnected if this endpoint's sample rate or channel count changes, in which case you will need to reconnect by calling this again.
	 */
	AUDIOEXTENSIONS_API Audio::FPatchInput PatchNewInput(float ExpectedDurationPerRender, float& OutSampleRate, int32& OutNumChannels);

	/**
	 * Post new settings for this endpoint.
	 * There is no type safety on this call, so make sure that you are using the correct implementation
	 * of IAudioEndpointSettingsProxy for this implementation of IAudioEndpoint.
	 */
	AUDIOEXTENSIONS_API void SetNewSettings(TUniquePtr<IAudioEndpointSettingsProxy>&& InNewSettings);

	/**
	 * If this audio endpoint hasn't spawned a seperate callback thread but requires a callback, this should be executed somewhere.
	 */
	AUDIOEXTENSIONS_API void ProcessAudioIfNeccessary();

	/**
	* Whether this endpoint is of an implemented type
	*/
	AUDIOEXTENSIONS_API virtual bool IsImplemented();

protected:

	/** REQUIRED OVERRIDES: */

	/** This should return the sample rate we should be sending to this endpoint. If the sample rate changes, please call DisconnectAllInputs(). */
	AUDIOEXTENSIONS_API virtual float GetSampleRate() const;

	/** This should return the number of channels we should be sending to this endpoint. If the number of channels changes, please call DisconnectAllInputs. */
	AUDIOEXTENSIONS_API virtual int32 GetNumChannels() const;

	/** OPTIONAL OVERRIDES: */

	/**
	 * For endpoints that do not explicitly fire a timed callback to poll audio data,
	 * this should be overridden to return true, and  OnAudioCallback and GetDesiredNumFrames should be overridden.
	 */
	virtual bool EndpointRequiresCallback() const { return false; }
	virtual int32 GetDesiredNumFrames() const { return 0; }

	/**
	 * For endpoints that override EndpointRequiresCallback to return true, 
	 * this callback will be called every (GetDesiredNumFrames() / GetSampleRate()) seconds.
	 * @returns whether the endpoint is still valid. If this returns false, DisconnectAllInputs will be called automatically.
	 */
	virtual bool OnAudioCallback(const TArrayView<const float>& InAudio, const int32& NumChannels, const IAudioEndpointSettingsProxy* InSettings) { return false; };

	/** METHODS USED BY IMPLEMENTATIONS OF IAudioEndpoint: */

	/**
	 * This is used by the IAudioEndpoint implementation to poll buffered audio to process or send to the endpoint.
	 * Note that this is NOT thread safe if EndpointRequiresCallback() is overridden to return true. If that is the case, use an override of OnAudioCallback instead.
	 * 
	 * @param [in] OutAudio: Pointer to already allocated buffer of floats, at least NumSamples long. This will be filled with interleaved audio based on GetNumChannels().
	 * @param [in] NumSamples: The number of samples to fill OutAudio with.
	 * @returns [out] the number of samples polled from this thing.
	 */
	AUDIOEXTENSIONS_API int32 PopAudio(float* OutAudio, int32 NumSamples);

	/**
	 * Use this as a thread safe way to use the current settings posted to this IAudioEndpoint. Locks with IAudioEndpoint::SetSettings.
	 * @param[in] NewSettingsRetrieved lambda used to work with the retrieved settings.
	 *                                 This lambda is called immediately and synchronously, but is used
	 *                                 to safely scope usage of the IAudioEndpointSettingsProxy pointer.
	 */
	AUDIOEXTENSIONS_API void PollSettings(TFunctionRef<void(const IAudioEndpointSettingsProxy*)> NewSettingsRetrieved);

	/**
	 * Thread safe function to disconnect everything from this endpoint. 
	 * Anything that owns an Audio::FPatchInput will be notified and will have to call PatchNewInput() again to reconnect.
	 */
	AUDIOEXTENSIONS_API void DisconnectAllInputs();

	/**
	 * If EndpointRequiresCallback() returns true, this can be used to spawn an async thread and begin calling OnAudioCallback.
	 */
	AUDIOEXTENSIONS_API void StartRunningAsyncCallback();
	AUDIOEXTENSIONS_API void StopRunningAsyncCallback();

	/**
	 * If EndpointRequiresCallback() returns true, this can be used to manually run the callback.
	 */
	AUDIOEXTENSIONS_API void RunCallbackSynchronously();


private:
	// Owns a scoped thread and runs OnAudioCallback when StartRunningCallback() is called.
	TUniquePtr<Audio::FMixerNullCallback> RenderCallback;
	
	// If we have a render callback, we pop audio from the PatchMixer to this buffer before calling OnAudioCallback.
	Audio::FAlignedFloatBuffer BufferForRenderCallback;

	// Owns the current settings for this endpoint.
	TUniquePtr<IAudioEndpointSettingsProxy> CurrentSettings;
	FCriticalSection CurrentSettingsCriticalSection;

	// Object used to mix all inputs together. Polled when OnAudioCallback is executed or when PollAudio is called.
	Audio::FPatchMixer PatchMixer;
};


/**
 * This factory is used to expose Endpoint types to the editor.
 * Once a factory is constructed and RegisterEndpointType is called, it will be exposed as a type of endpoint
 * That a submix in the submix graph could be constructed with.
 */
class IAudioEndpointFactory : public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioEndpointFactory() = default;

	/** Get the name for the endpoint type that this factory produces.  */
	AUDIOEXTENSIONS_API virtual FName GetEndpointTypeName();

	/** This is a special cased name for endpoint submixes that render directly to the default audio device in Audio::FMixerDevice::OnProcessAudioStream. */
	static AUDIOEXTENSIONS_API FName GetTypeNameForDefaultEndpoint();

	/** 
	 * This is used when calling IModularFeatures::Get().RegisterModularFeature for IAudioEndpointFactory implementations. 
	 * It's not needed if one uses RegisterEndpointType() to register IAudioEndpointFactory implementations. 
	 */
	static AUDIOEXTENSIONS_API FName GetModularFeatureName();

	/** 
	 * This needs to be called to make a soundfield format usable by the engine.
	 * It can be called from a ISoundfieldFactory subclass' constructor
	*/
	static AUDIOEXTENSIONS_API void RegisterEndpointType(IAudioEndpointFactory* InFactory);

	/**
	 * This needs to be called it an implementation of ISoundfieldFactory is about to be destroyed.
	 * It can be called from the destructor of an implementation of ISoundfieldFactory.
	 */
	static AUDIOEXTENSIONS_API void UnregisterEndpointType(IAudioEndpointFactory* InFactory);

	/**
	 * Get a registered endpoint factory by name.
	 */
	static AUDIOEXTENSIONS_API IAudioEndpointFactory* Get(const FName& InName);

	static AUDIOEXTENSIONS_API TArray<FName> GetAvailableEndpointTypes();

	/** Called for every new endpoint submix created with this factory's endpoint type. */
	AUDIOEXTENSIONS_API virtual TUniquePtr<IAudioEndpoint> CreateNewEndpointInstance(const FAudioPluginInitializationParams& InitInfo, const IAudioEndpointSettingsProxy& InitialSettings);

	/**
	 * Should return the StaticClass of this factory's implementation of UAudioEndpointSettingsBase.
	 */
	AUDIOEXTENSIONS_API virtual UClass* GetCustomSettingsClass() const;

	/**
	 * return the settings an endpoint should use 
	 */
	AUDIOEXTENSIONS_API virtual const UAudioEndpointSettingsBase* GetDefaultSettings() const;

	bool bIsImplemented = false;

	static AUDIOEXTENSIONS_API IAudioEndpointFactory* GetDummyFactory();
};
