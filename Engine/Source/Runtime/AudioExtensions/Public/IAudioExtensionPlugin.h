// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"
#include "HAL/LowLevelMemTracker.h"
#include "ISoundfieldFormat.h"
#include "Math/Interval.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/ObjectMacros.h"
#include "AudioDefines.h"
#include "IAudioProxyInitializer.h"

#include "IAudioExtensionPlugin.generated.h"

// Forward Declarations
class FAudioDevice;
class FSoundEffectBase;
class FSoundEffectSource;
class FSoundEffectSubmix;
struct FWaveInstance;
class IAudioModulationManager;
class IAudioOcclusion;
class IAudioPluginListener;
class IAudioReverb;
class IAudioSourceDataOverride;
class IAudioSpatialization;
class USoundSubmix;

LLM_DECLARE_TAG_API(Audio_SpatializationPlugins, AUDIOEXTENSIONS_API);
// Convenience macro for Audio_SpatializationPlugins LLM scope to avoid misspells.
#define AUDIO_SPATIALIZATION_PLUGIN_LLM_SCOPE LLM_SCOPE_BYTAG(Audio_SpatializationPlugins);

/**
* Enumeration of audio plugin types
*
*/
enum class EAudioPlugin : uint8
{
	SPATIALIZATION = 0,
	REVERB = 1,
	OCCLUSION = 2,
	MODULATION = 3,
	SOURCEDATAOVERRIDE = 4,

	COUNT = 5
};


// Deprecated in favor of TSoundEffectSubmixPtr
using FSoundEffectSubmixPtr   = TSharedPtr<FSoundEffectSubmix, ESPMode::ThreadSafe>;

using TSoundEffectPtr		  = TSharedPtr<FSoundEffectBase, ESPMode::ThreadSafe>;
using TSoundEffectSourcePtr   = TSharedPtr<FSoundEffectSource, ESPMode::ThreadSafe>;
using TSoundEffectSubmixPtr   = TSharedPtr<FSoundEffectSubmix, ESPMode::ThreadSafe>;
using TAudioSpatializationPtr = TSharedPtr<IAudioSpatialization, ESPMode::ThreadSafe>;
using TAudioSourceDataOverridePtr = TSharedPtr<IAudioSourceDataOverride, ESPMode::ThreadSafe>;
using TAudioModulationPtr     = TSharedPtr<IAudioModulationManager, ESPMode::ThreadSafe>;
using TAudioOcclusionPtr      = TSharedPtr<IAudioOcclusion, ESPMode::ThreadSafe>;
using TAudioReverbPtr         = TSharedPtr<IAudioReverb, ESPMode::ThreadSafe>;
using TAudioPluginListenerPtr = TSharedPtr<IAudioPluginListener, ESPMode::ThreadSafe>;

/**
* FSpatializationParams
* Struct for retrieving parameters needed for computing spatialization and occlusion plugins.
*/
struct FSpatializationParams
{
	/** The listener position (is likely at the origin). */
	FVector ListenerPosition;

	/** The listener orientation. */
	FQuat ListenerOrientation;

	/** The emitter position relative to listener. */
	FVector EmitterPosition;

	/** The emitter world position. */
	FVector EmitterWorldPosition;

	/** The emitter world rotation. */
	FQuat EmitterWorldRotation;

	/** The emitter world rotation on callback ago. */
	FQuat LastEmitterWorldRotation;

	/** The left channel position. */
	FVector LeftChannelPosition;

	/** The right channel position. */
	FVector RightChannelPosition;

	/** The distance between listener and emitter. */
	float Distance;

	/** The distance used to compute attenuation. Maybe different from the distance between listener and emitter if it's overridden. */
	float AttenuationDistance;

	/** Deprecated */
	float NormalizedOmniRadius;

	/** The amount of non-spatialized this source is. 1.0 means fully 2D, 0.0 means fully 3D. */
	float NonSpatializedAmount;

	/** The time when this spatialization params was built. */
	double AudioClock;

	FSpatializationParams()
		: ListenerPosition(FVector::ZeroVector)
		, ListenerOrientation(FQuat::Identity)
		, EmitterPosition(FVector::ZeroVector)
		, EmitterWorldPosition(FVector::ZeroVector)
		, EmitterWorldRotation(FQuat::Identity)
		, LastEmitterWorldRotation(FQuat::Identity)
		, LeftChannelPosition(FVector::ZeroVector)
		, RightChannelPosition(FVector::ZeroVector)
		, Distance(0.0f)
		, AttenuationDistance(0.0f)
		, NormalizedOmniRadius(0.0f)
		, NonSpatializedAmount(0.0f)
		, AudioClock(0.0)
	{}
};

struct FAudioPluginInitializationParams
{
	//Maximum number of sources that can play simultaneously.
	uint32 NumSources;

	//Number of output channels.
	uint32 NumOutputChannels;

	//Sample rate.
	uint32 SampleRate;

	//Buffer length used for each callback.
	uint32 BufferLength;

	//Pointer to audio device owning this audio plugin.
	//IMPORTANT: This will be deprecated once the AudioMixer
	//           is taken out of the experimental branch.
	FAudioDevice* AudioDevicePtr;

	FAudioPluginInitializationParams()
		: NumSources(0)
		, NumOutputChannels(0)
		, SampleRate(0)
		, BufferLength(0)
		, AudioDevicePtr(nullptr)
	{}

};

struct FAudioPluginSourceInputData
{
	// The index of the source voice. Guaranteed to be between 0 and the max number of sources rendered.
	int32 SourceId;

	// The ID of the audio component associated with the wave instance.
	uint64 AudioComponentId;

	// The audio input buffer
	Audio::FAlignedFloatBuffer* AudioBuffer;

	// Number of channels of the source audio buffer.
	int32 NumChannels;

	// The listener orientation.
	FQuat ListenerOrientation;

	// Spatialization parameters.
	const FSpatializationParams* SpatializationParams;
};

struct FAudioPluginSourceOutputData
{
	// The audio output buffer
	Audio::FAlignedFloatBuffer AudioBuffer;
};

/** This is a class which should be overridden to provide users with settings to use for individual sounds */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USpatializationPluginSourceSettingsBase : public UObject
{
	GENERATED_BODY()
};

/************************************************************************/
/* IAudioPluginFactory                                             */
/* This interface is inherited by spatialization, reverb and occlusion  */
/* plugins to describe specifics of a plugin such as platform support,  */
/* and display names.                                                   */
/************************************************************************/
class IAudioPluginFactory
{
public:
	/*
	* Returns human-readable string representing the display name of this plugin.
	* This is the name that will be used in settings and .ini files.
	* If multiple IAudioPlugin implementations are found that return identical strings here,
	* The first one of these loaded will be used.
	*
	* @return FString of the display name of this plugin.
	*/
	virtual FString GetDisplayName()
	{
		static FString DisplayName = FString("Generic Audio Plugin");
		return DisplayName;
	}

	/*
	* Returns whether this plugin supports use on the specified platform.
	* @param Platform an enumerated platform (i.e. Windows, Playstation4, etc.)
	* @return true if this plugin supports use on Platform, false otherwise.
	*/
	virtual bool SupportsPlatform(const FString& PlatformName) = 0;

	/*
	* Returns whether this plugin sends audio to an external renderer.
	* if this returns true, the audio engine will not mix the result of the audio process callback
	* from the plugin into the audio output.
	*
	* @return true only if the plugin will handle sending audio to the DAC itself.
	*/
	virtual bool IsExternalSend()
	{
		return false;
	}

	/*
	*  @return true if the plugin returns from its external submix to a submix in the plugin.
	*/
	virtual bool ReturnsToSubmixGraph() const
	{
		return false;
	}
};

/************************************************************************/
/* IAudioSpatializationFactory                                          */
/* Implement this modular feature to make your Spatialization plugin */
/* visible to the engine.                                               */
/************************************************************************/
class IAudioSpatializationFactory : public IAudioPluginFactory, public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioSpatializationFactory() = default;

	// IModularFeature
	static FName GetModularFeatureName()
	{
		static FName AudioExtFeatureName = FName("AudioSpatializationPlugin");
		return AudioExtFeatureName;
	}

	/* Begin IAudioPluginWithMetadata implementation */
	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString("Generic Audio Spatialization Plugin");
		return DisplayName;
	}
	/* End IAudioPluginWithMetadata implementation */

	/**
	* @return the max amount of channels your plugin supports. For example, override this to
	*         return 2 to support spatializing mono and stereo sources.
	*/
	virtual int32 GetMaxSupportedChannels()
	{
		return 1;
	}

	/**
	* @return a new instance of your spatialization plugin, owned by a shared pointer.
	*/
	virtual TAudioSpatializationPtr CreateNewSpatializationPlugin(FAudioDevice* OwningDevice) = 0;

	/** 
	* @return the UClass type of your settings for spatialization. This allows us to only pass in user settings for your plugin.
	*/
	virtual UClass* GetCustomSpatializationSettingsClass() const
	{
		return nullptr;
	}
};

/**
* IAudioSpatialization
*
* This class represents instances of a plugin that will process spatialization for a stream of audio.
* Currently used to process a mono-stream through an HRTF spatialization algorithm into a stereo stream.
* This algorithm contains an audio effect assigned to every VoiceId (playing sound instance). It assumes
* the effect is updated in the audio engine update loop with new position information.
*
*/
class IAudioSpatialization 
{
public:
	/** Virtual destructor */
	virtual ~IAudioSpatialization() = default;

	/**
	* Shuts down the audio plugin.
	*
	*/
	virtual void Shutdown()
	{
	}

	virtual void OnDeviceShutdown(FAudioDevice* AudioDevice)
	{
	}

	/** DEPRECATED: sets the spatialization effect parameters. */
	virtual void SetSpatializationParameters(uint32 SourceId, const FSpatializationParams& Params)
	{
	}

	/** DEPRECATED: Gets the spatialization effect parameters. */
	virtual void GetSpatializationParameters(uint32 SourceId, FSpatializationParams& OutParams)
	{
	}
	
	/** DEPRECATED: Initializes the spatialization effect with the given buffer length. */
	virtual void InitializeSpatializationEffect(uint32 BufferLength)
	{
	}

	/** DEPRECATED: Uses the given HRTF algorithm to spatialize a mono audio stream. */
	virtual void ProcessSpatializationForVoice(uint32 SourceId, float* InSamples, float* OutSamples, const FVector& Position)
	{
	}

	/** DEPRECATED: Uses the given HRTF algorithm to spatialize a mono audio stream, assumes the parameters have already been set before processing. */
	virtual void ProcessSpatializationForVoice(uint32 SourceId, float* InSamples, float* OutSamples)
	{
	}

	/** Called when a source is assigned to a voice. */
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USpatializationPluginSourceSettingsBase* InSettings)
	{
	}

	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, USpatializationPluginSourceSettingsBase* InSettings)
	{
		OnInitSource(SourceId, AudioComponentUserId, InSettings);
	}

	/** Called when a source is done playing and is released. */
	virtual void OnReleaseSource(const uint32 SourceId)
	{
	}

	/** Processes audio with the given input and output data structs.*/
	virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
	}

	/** Called when all sources have finished processing. */
	virtual void OnAllSourcesProcessed()
	{
	}

	/** Returns whether or not the spatialization effect has been initialized */
	virtual bool IsSpatializationEffectInitialized() const
	{
		return false;
	}

	/** Initializes the spatialization plugin with the given buffer length. */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
	}

	/** Creates an audio spatialization effect. */
	virtual bool CreateSpatializationEffect(uint32 SourceId)
	{
		return true;
	}

	/**	Returns the spatialization effect for the given voice id. */
	virtual void* GetSpatializationEffect(uint32 SourceId)
	{
		return nullptr;
	}
};


/** This is a class which should be overridden to provide users with settings to use for individual sounds */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USourceDataOverridePluginSourceSettingsBase : public UObject
{
	GENERATED_BODY()
};

/************************************************************************/
/* IAudioSourceDataOverrideFactory										*/
/* Implement this modular feature to make your SourceDataOverride plugin*/
/* visible to the engine.                                               */
/************************************************************************/
class IAudioSourceDataOverrideFactory : public IAudioPluginFactory, public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioSourceDataOverrideFactory() = default;

	// IModularFeature
	static FName GetModularFeatureName()
	{
		static FName AudioExtFeatureName = FName("AudioSourceDataOverridePlugin");
		return AudioExtFeatureName;
	}

	/* Begin IAudioPluginWithMetadata implementation */
	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString("Generic Audio Source Data Override Plugin");
		return DisplayName;
	}
	/* End IAudioPluginWithMetadata implementation */

		/**
	* @return the UClass type of your settings for source data overrides. This allows us to only pass in user settings for your plugin.
	*/
	virtual UClass* GetCustomSourceDataOverrideSettingsClass() const
	{
		return nullptr;
	}

	/**
	* @return a new instance of your source data override plugin, owned by a shared pointer.
	*/
	virtual TAudioSourceDataOverridePtr CreateNewSourceDataOverridePlugin(FAudioDevice* OwningDevice) = 0;
};


/** Interface to allow a plugin to override a sound's actual position and simulate propagation (e.g. traversal around corners, etc). */
class IAudioSourceDataOverride
{
public:
	/** Virtual destructor */
	virtual ~IAudioSourceDataOverride() = default;

	/** Initializes the source data override plugin with the given buffer length. */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
	}

	/** Called when a source is assigned to a voice. */
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, USourceDataOverridePluginSourceSettingsBase* InSettings)
	{
	}

	/** Called when a source is done playing and is released. */
	virtual void OnReleaseSource(const uint32 SourceId)
	{
	}

	/** Allows this plugin to override any source data. Called per audio source before any other parameters are updated on sound sources. */
	virtual void GetSourceDataOverrides(const uint32 SourceId, const FTransform& InListenerTransform, FWaveInstance* InOutWaveInstance)
	{
	}

	/** Called when all sources have finished processing. */
	virtual void OnAllSourcesProcessed()
	{
	}
};

/** This is a class which should be overridden to provide users with settings to use for individual sounds */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class UOcclusionPluginSourceSettingsBase : public UObject
{
	GENERATED_BODY()
};

/************************************************************************/
/* IAudioOcclusionFactory                                               */
/*                                                                      */
/************************************************************************/
class IAudioOcclusionFactory : public IAudioPluginFactory, public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioOcclusionFactory() = default;

	// IModularFeature
	static FName GetModularFeatureName()
	{
		static FName AudioExtFeatureName = FName("AudioOcclusionPlugin");
		return AudioExtFeatureName;
	}

	/* Begin IAudioPluginWithMetadata implementation */
	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString("Generic Audio Occlusion Plugin");
		return DisplayName;
	}
	/* End IAudioPluginWithMetadata implementation */

	virtual TAudioOcclusionPtr CreateNewOcclusionPlugin(FAudioDevice* OwningDevice) = 0;

	/**
	* @return the UClass type of your settings for occlusion. This allows us to only pass in user settings for your plugin.
	*/
	virtual UClass* GetCustomOcclusionSettingsClass() const
	{
		return nullptr;
	}
};

class IAudioOcclusion
{
public:
	/** Virtual destructor */
	virtual ~IAudioOcclusion() = default;

	/** Initialize the occlusion plugin with the same rate and number of sources. */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
	}

	/**
	* Shuts down the audio plugin.
	*
	*/
	virtual void Shutdown()
	{
	}

	/** Called when a source is assigned to a voice. */
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, UOcclusionPluginSourceSettingsBase* InSettings)
	{
	}

	/** Called when a source is done playing and is released. */
	virtual void OnReleaseSource(const uint32 SourceId)
	{
	}

	/** Processes audio with the given input and output data structs.*/
	virtual void ProcessAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
	}
};

/************************************************************************/
/* IAudioModulationFactory                                              */
/*                                                                      */
/************************************************************************/
class IAudioModulationFactory : public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioModulationFactory() = default;

	// IModularFeature
	static FName GetModularFeatureName()
	{
		static FName AudioExtFeatureName = FName("AudioModulationPlugin");
		return AudioExtFeatureName;
	}

	virtual const FName& GetDisplayName() const = 0;

	virtual TAudioModulationPtr CreateNewModulationPlugin(FAudioDevice* OwningDevice) = 0;

	/**
	* @return the UClass type of your settings for modulation. This allows us to only pass in user settings for your plugin.
	*/
	virtual UClass* GetCustomModulationSettingsClass() const
	{
		return nullptr;
	}
};


/** This is a class which should be overridden to provide users with settings to use for individual sounds */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class UReverbPluginSourceSettingsBase : public UObject
{
	GENERATED_BODY()
};
 
class IAudioReverbFactory : public IAudioPluginFactory, public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~IAudioReverbFactory() = default;

	// IModularFeature
	static FName GetModularFeatureName()
	{
		static FName AudioExtFeatureName = FName("AudioReverbPlugin");
		return AudioExtFeatureName;
	}

	/* Begin IAudioPluginWithMetadata implementation */
	virtual FString GetDisplayName() override
	{
		static FString DisplayName = FString("Generic Audio Reverb Plugin");
		return DisplayName;
	}
	/* End IAudioPluginWithMetadata implementation */

	virtual TAudioReverbPtr CreateNewReverbPlugin(FAudioDevice* OwningDevice) = 0;

	/**
	* @return the UClass type of your settings for reverb. This allows us to only pass in user settings for your plugin.
	*/
	virtual UClass* GetCustomReverbSettingsClass() const
	{
		return nullptr;
	}
};

class IAudioReverb
{
public:
	/** Virtual destructor */
	virtual ~IAudioReverb() = default;

	/** Initialize the reverb plugin with the same rate and number of sources. */
	virtual void Initialize(const FAudioPluginInitializationParams InitializationParams)
	{
	}

	/**
	* Shuts down the audio plugin.
	*
	*/
	virtual void Shutdown()
	{
	}

	virtual void OnDeviceShutdown(FAudioDevice* AudioDevice)
	{
	}

	/** Called when a source is assigned to a voice. */
	virtual void OnInitSource(const uint32 SourceId, const FName& AudioComponentUserId, const uint32 NumChannels, UReverbPluginSourceSettingsBase* InSettings) = 0;

	/** Called when a source is done playing and is released. */
	virtual void OnReleaseSource(const uint32 SourceId) = 0;

	/** Returns the plugin-managed effect submix instance */
	virtual FSoundEffectSubmixPtr GetEffectSubmix() = 0;

	virtual USoundSubmix* LoadSubmix()
	{
		return GetSubmix();
	}

	/** Returns the plugin-managed effect submix */
	virtual USoundSubmix* GetSubmix() = 0;

	/** Processes audio with the given input and output data structs.*/
	virtual void ProcessSourceAudio(const FAudioPluginSourceInputData& InputData, FAudioPluginSourceOutputData& OutputData)
	{
	}
};

/************************************************************************/
/* IAudioPluginListener                                                 */
/* Implementations of this interface can receive updates about the      */
/* audio listener's position in the game world, as well as other data.  */
/* to use this, register a ListenerObserver to an audio device using    */
/* FAudioDevice::RegisterPluginListener().                              */
/************************************************************************/
class IAudioPluginListener
{
public:
	virtual ~IAudioPluginListener() = default;

	virtual void OnDeviceShutdown(FAudioDevice* AudioDevice)
	{
	}

	//This function is called when a game world initializes a listener with an audio device this
	//IAudioPluginListener is registered to. Please note that it is possible to miss this event
	//if you register this IAudioPluginListener after the listener is initialized.
	virtual void OnListenerInitialize(FAudioDevice* AudioDevice, UWorld* ListenerWorld)
	{
	}

	// This is overridable for any actions a plugin manager may need to do on the game thread.
	virtual void OnTick(UWorld* InWorld, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds)
	{
	}

	// This is overridable for any actions a plugin manager may need to do on a level change.
	virtual void OnWorldChanged(FAudioDevice* AudioDevice, UWorld* InWorld)
	{
	}

	// Called when the listener is updated on the given audio device.
	virtual void OnListenerUpdated(FAudioDevice* AudioDevice, const int32 ViewportIndex, const FTransform& ListenerTransform, const float InDeltaSeconds)
	{
	}

	//Called when the listener is shutdown.
	virtual void OnListenerShutdown(FAudioDevice* AudioDevice)
	{
	}
};
