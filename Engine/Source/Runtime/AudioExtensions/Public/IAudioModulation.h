// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioDefines.h"
#include "CoreTypes.h"
#include "DSP/BufferVectorOperations.h"
#include "IAudioExtensionPlugin.h"
#include "IAudioProxyInitializer.h"
#include "Internationalization/Text.h"
#include "Math/MathFwd.h"
#include "Math/Rotator.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IAudioModulation.generated.h"

// Forward Declarations
class IAudioModulationManager;
class ISoundModulatable;
class UObject;
class USoundModulatorBase;
struct FAudioPluginInitializationParams;
struct FAudioPluginSourceInputData;
struct FAudioPluginSourceOutputData;

#if !UE_BUILD_SHIPPING
class FCanvas;
class FCommonViewportClient;
class FViewport;
class UFont;
#endif // !UE_BUILD_SHIPPING

namespace Audio
{
	using FModulatorId = uint32;
	using FModulatorTypeId = uint32;
	using FModulatorHandleId = uint32;

	using FModulationUnitConversionFunction = TFunction<void(float& /* OutValueNormalizedToUnit */)>;
	using FModulationNormalizedConversionFunction = TFunction<void(float& /* OutValueUnitToNormalized */)>;
	using FModulationMixFunction = TFunction<void(float& /* OutNormalizedA */, float /* InNormalizedB */)>;


	struct FModulationParameter
	{
		AUDIOEXTENSIONS_API FModulationParameter();
		FModulationParameter(const FModulationParameter& InParam) = default;
		FModulationParameter(FModulationParameter&& InParam) = default;

		FModulationParameter& operator=(FModulationParameter&& InParam) = default;
		FModulationParameter& operator=(const FModulationParameter& InParam) = default;

		FName ParameterName;

		// Default value of parameter in unit space
		float DefaultValue = 1.0f;

		// Default minimum value of parameter in unit space
		float MinValue = 0.0f;

		// Default minimum value of parameter in unit space
		float MaxValue = 1.0f;

		// Whether or not unit conversion is required
		bool bRequiresConversion = false;

		// Whether or not to clamp normalized values to [0,1] when using this parameter. 
		bool bClampToNormalizedRange = true;
		
		uint32 TypeHash = INDEX_NONE;

#if WITH_EDITORONLY_DATA
		FText UnitDisplayName;

		FName ClassName;
#endif // WITH_EDITORONLY_DATA

		// Function used to mix normalized values together.
		FModulationMixFunction MixFunction;

		// Function used to convert value buffer from normalized, unitless space [0.0f, 1.0f] to unit space.
		FModulationUnitConversionFunction UnitFunction;

		// Function used to convert value buffer from unit space to normalized, unitless [0.0f, 1.0f] space.
		FModulationNormalizedConversionFunction NormalizedFunction;

		static AUDIOEXTENSIONS_API const FModulationMixFunction& GetDefaultMixFunction();
		static AUDIOEXTENSIONS_API const FModulationUnitConversionFunction& GetDefaultUnitConversionFunction();
		static AUDIOEXTENSIONS_API const FModulationNormalizedConversionFunction& GetDefaultNormalizedConversionFunction();

		friend inline uint32 GetTypeHash(const FModulationParameter& InModulationParameter)
		{
			return InModulationParameter.TypeHash;
		}
	};

	AUDIOEXTENSIONS_API bool IsModulationParameterRegistered(FName InName);
	AUDIOEXTENSIONS_API void RegisterModulationParameter(FName InName, FModulationParameter&& InParameter);
	AUDIOEXTENSIONS_API bool UnregisterModulationParameter(FName InName);
	AUDIOEXTENSIONS_API void UnregisterAllModulationParameters();
	AUDIOEXTENSIONS_API const FModulationParameter* GetModulationParameterPtr(FName InName);
	AUDIOEXTENSIONS_API const FModulationParameter& GetDefaultModulationParameter();
	
	UE_INTERNAL
	AUDIOEXTENSIONS_API void SetModulationParameterClampOverride(FName InName, bool bClamp);
	
	UE_INTERNAL
	AUDIOEXTENSIONS_API void ClearModulationParameterClampOverride(FName InName);
	
	UE_INTERNAL
	AUDIOEXTENSIONS_API void ClearAllModulationParameterClampOverrides();

	// Returns false if parameter is not registered
	UE_INTERNAL
	AUDIOEXTENSIONS_API bool GetModulationParameterClampValue(const FName& ParameterName, bool& bClampOut);
	
	UE_DEPRECATED(5.6, "Please use the version of GetModulationParameter which returns a pointer instead of a reference, and check that it is valid in your code.")
	AUDIOEXTENSIONS_API const FModulationParameter& GetModulationParameter(FName InName);

	/** Interface for cached off Modulator UObject data used as default settings to
	  * be converted to instanced proxy data per AudioDevice on the AudioRenderThread.
	  * If proxy is already active, implementation is expected to ignore register call
	  * and return existing modulator proxy's type Id & set parameter accordingly.
	  */
	class IModulatorSettings
	{
	public:
		virtual ~IModulatorSettings() = default;
		virtual TUniquePtr<IModulatorSettings> Clone() const = 0;
		virtual FModulatorId GetModulatorId() const = 0;
		virtual const Audio::FModulationParameter& GetOutputParameter() const = 0;
		virtual Audio::FModulatorTypeId Register(
			Audio::FModulatorHandleId HandleId,
			IAudioModulationManager& InModulation) const = 0;
	};

	/** Handle to a modulator which interacts with the modulation API to manage lifetime
	  * of modulator proxy objects internal to modulation plugin implementation.
	  */
	struct FModulatorHandle
	{
		FModulatorHandle() = default;
		AUDIOEXTENSIONS_API FModulatorHandle(Audio::FModulationParameter&& InParameter);
		AUDIOEXTENSIONS_API FModulatorHandle(IAudioModulationManager& InModulation, const Audio::IModulatorSettings& InModulatorSettings, Audio::FModulationParameter&& InParameter);
		AUDIOEXTENSIONS_API FModulatorHandle(const FModulatorHandle& InOther);
		AUDIOEXTENSIONS_API FModulatorHandle(FModulatorHandle&& InOther);

		AUDIOEXTENSIONS_API ~FModulatorHandle();

		AUDIOEXTENSIONS_API FModulatorHandle& operator=(const FModulatorHandle& InOther);
		AUDIOEXTENSIONS_API FModulatorHandle& operator=(FModulatorHandle&& InOther);

		AUDIOEXTENSIONS_API FModulatorId GetModulatorId() const;
		AUDIOEXTENSIONS_API const FModulationParameter& GetParameter() const;
		AUDIOEXTENSIONS_API FModulatorTypeId GetTypeId() const;
		AUDIOEXTENSIONS_API FModulatorHandleId GetHandleId() const;
		AUDIOEXTENSIONS_API bool GetValue(float& OutValue) const;
		AUDIOEXTENSIONS_API bool GetValueThreadSafe(float& OutValue) const;
		AUDIOEXTENSIONS_API bool IsValid() const;

		friend inline uint32 GetTypeHash(const FModulatorHandle& InModulatorHandle)
		{
			return HashCombineFast(InModulatorHandle.HandleId, InModulatorHandle.ModulatorId);
		}

		inline bool operator==(const FModulatorHandle& Other) const
		{
			return HandleId == Other.HandleId && ModulatorId == Other.ModulatorId;
		}

		inline bool operator!=(const FModulatorHandle& Other) const
		{
			return !(*this == Other);
		}

	private:
		FModulationParameter Parameter;
		FModulatorHandleId HandleId = (FModulatorHandleId)INDEX_NONE;
		FModulatorTypeId ModulatorTypeId = (FModulatorTypeId)INDEX_NONE;
		FModulatorId ModulatorId = (FModulatorId)INDEX_NONE;
		TWeakPtr<IAudioModulationManager> Modulation;
	};
} // namespace Audio

class IAudioModulationManager : public TSharedFromThis<IAudioModulationManager>
{
public:
	/** Virtual destructor */
	virtual ~IAudioModulationManager() = default;

	/** Initialize the modulation plugin with the same rate and number of sources */
	virtual void Initialize(const FAudioPluginInitializationParams& InitializationParams) = 0;

	virtual void OnAuditionEnd() = 0;

	virtual void GetControlBusAssetPath(TArray<FTopLevelAssetPath>& OutClassNames) = 0;

#if !UE_BUILD_SHIPPING
	/** Request to post help from active plugin (non-shipping builds only) */
	virtual bool OnPostHelp(FCommonViewportClient* ViewportClient, const TCHAR* Stream) = 0; 

	/** Render stats pertaining to modulation (non-shipping builds only) */
	virtual int32 OnRenderStat(FViewport* Viewport, FCanvas* Canvas, int32 X, int32 Y, const UFont& Font, const FVector* ViewLocation, const FRotator* ViewRotation) = 0;

	/** Toggle showing render stats pertaining to modulation (non-shipping builds only) */
	virtual bool OnToggleStat(FCommonViewportClient* ViewportClient, const TCHAR* Stream) = 0;
#endif //!UE_BUILD_SHIPPING

	/** Processes all modulators Run on the audio render thread prior to processing audio */
	virtual void ProcessModulators(const double InElapsed) = 0;

	/** Updates modulator definition on the AudioRender Thread with that provided by the UObject representation */
	virtual void UpdateModulator(const USoundModulatorBase& InModulator) = 0;

protected:
	virtual void RegisterModulator(uint32 InHandleId, Audio::FModulatorId InModulatorId) = 0;

	// Get the modulator value from the AudioRender Thread
	virtual bool GetModulatorValue(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const = 0;

	// Get the modulator value from any thread.
	virtual bool GetModulatorValueThreadSafe(const Audio::FModulatorHandle& ModulatorHandle, float& OutValue) const = 0;

	virtual void UnregisterModulator(const Audio::FModulatorHandle& InHandle) = 0;

	friend Audio::FModulatorHandle;
};

/**
 * Base class for all modulators
 */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundModulatorBase : public UObject, public IAudioProxyDataFactory
{
	GENERATED_BODY()

public:
	AUDIOEXTENSIONS_API virtual const Audio::FModulationParameter& GetOutputParameter() const;

	AUDIOEXTENSIONS_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams) override;

	AUDIOEXTENSIONS_API virtual TUniquePtr<Audio::IModulatorSettings> CreateProxySettings() const;
};

/** Proxy to modulator, allowing for modulator to be referenced by the Audio Render Thread independently
  * from the implementing modulation plugin (ex. for MetaSound implementation).
  */
class FSoundModulatorAssetProxy : public Audio::TProxyData<FSoundModulatorAssetProxy>, public TSharedFromThis<FSoundModulatorAssetProxy, ESPMode::ThreadSafe>
{
public:
	IMPL_AUDIOPROXY_CLASS(FSoundModulatorAssetProxy);

	FSoundModulatorAssetProxy(const FSoundModulatorAssetProxy& InAssetProxy)
		: Parameter(InAssetProxy.Parameter)
		, ModulatorSettings(InAssetProxy.ModulatorSettings.IsValid() ? InAssetProxy.ModulatorSettings->Clone() : nullptr)
	{
	}

	FSoundModulatorAssetProxy(const USoundModulatorBase& InModulatorBase)
		: Parameter(InModulatorBase.GetOutputParameter())
		, ModulatorSettings(InModulatorBase.CreateProxySettings())
	{
	}

	virtual Audio::FModulatorHandle CreateModulatorHandle(IAudioModulationManager& InModulation) const
	{
		check(ModulatorSettings.IsValid());

		Audio::FModulationParameter HandleParameter = Parameter;
		return Audio::FModulatorHandle(InModulation, *ModulatorSettings.Get(), MoveTemp(HandleParameter));
	}

	virtual Audio::FModulatorId GetModulatorId() const
	{
		check(ModulatorSettings.IsValid())
		return ModulatorSettings->GetModulatorId();
	}

protected:
	Audio::FModulationParameter Parameter;
	TUniquePtr<Audio::IModulatorSettings> ModulatorSettings;
};
using FSoundModulatorAssetProxyPtr = TSharedPtr<FSoundModulatorAssetProxy, ESPMode::ThreadSafe>;

/** Proxy to modulator, allowing for modulator to be referenced by the Audio Render Thread independently
  * from the implementing modulation plugin (ex. for MetaSound implementation).
  */
class FSoundModulationParameterAssetProxy : public Audio::TProxyData<FSoundModulationParameterAssetProxy>, public TSharedFromThis<FSoundModulationParameterAssetProxy, ESPMode::ThreadSafe>
{
public:
	IMPL_AUDIOPROXY_CLASS(FSoundModulationParameterAssetProxy);

	virtual const Audio::FModulationParameter& GetParameter() const
	{
		return Parameter;
	}

protected:
	Audio::FModulationParameter Parameter;
};
using FSoundModulationParameterAssetProxyPtr = TSharedPtr<FSoundModulationParameterAssetProxy, ESPMode::ThreadSafe>;

/** Interface to sound that is modulatable, allowing for certain specific
  * behaviors to be controlled on the sound level by the modulation system.
  */
class ISoundModulatable
{
public:
	virtual ~ISoundModulatable() = default;

	/**
	 * Gets the object definition id of the given playing sound's instance
	 */
	virtual uint32 GetObjectId() const = 0;

	/**
	 * Returns number of actively instances of sound playing (including virtualized instances)
	 */
	virtual int32 GetPlayCount() const = 0;

	/**
	 * Returns whether or not sound is an editor preview sound
	 */
	virtual bool IsPreviewSound() const = 0;

	/**
	 * Stops sound.
	 */
	virtual void Stop() = 0;
};
