// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameter.h"
#include "AudioParameterControllerInterface.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"


namespace Audio
{
	// Forward Declarations
	class IAudioParameterInterfaceRegistry;

	// Interface for parameterizing data provided to or coming from an executable audio unit
	// (ex. Sound like a MetaSoundSource, arbitrary DSP graph like a MetaSoundPatch, etc.).
	// Can be used generically for processing either logically at a control or Game Thread
	// tick rate (ex. SoundCues), or by an underlying DSP operation via the Audio Render
	// Thread or delegated task (ex. MetaSounds, custom SoundGenerator, etc.)
	struct FParameterInterface
	{
		// Version of interface (higher numbers are more recent)
		struct FVersion
		{
			const int32 Major = 1;
			const int32 Minor = 0;
		};

		// Input of interface
		struct FInput
		{
			// Name to be displayed in editor or tools
			const FText DisplayName;

			// Description to be displayed in editor or tools
			const FText Description;

			// FName describing the type of the data.  May be
			// interpreted solely by a plugin or implementation
			// (ex. MetaSounds).  If blank, type is assumed to be
			// that described by the InitValue AudioParameter's
			// corresponding ParamType. If provided, DataType
			// must be constructible using the InitValue.
			const FName DataType;

			// Initial value of the given parameter
			const FAudioParameter InitValue;

			// Text to display in the editor or tools if the consuming
			// system of the given input parameter is not implemented (to avoid
			// passing data using the parameter system when not operated on).
			const FText RequiredText;

			// Visual sort order of the given input with respect to other inputs either
			// within the given interface or listed among other unrelated inputs.
			const int32 SortOrderIndex = 0;
		};

		// Output of interface
		struct FOutput
		{
			// Name to be displayed in editor or tools
			const FText DisplayName;

			// Description to be displayed in editor or tools
			const FText Description;

			// FName describing the type of the data.  May be
			// interpreted solely by a plugin or implementation
			// (ex. MetaSounds).  If blank, type is assumed to be
			// that described by ParamType.
			const FName DataType;

			// Name of output parameter used as a runtime identifier
			const FName ParamName;

			// Text to display in the editor or tools if the consuming
			// system of the given input parameter is not implemented (to avoid
			// passing data using the parameter system when not operated on).
			const FText RequiredText = FText();

			// Type of output parameter used as a runtime identifier if unspecified by the DataType.
			const EAudioParameterType ParamType = EAudioParameterType::None;

			// Visual sort order of the given input with respect to other outputs either
			// within the given interface or listed among other unrelated outputs.
			const int32 SortOrderIndex = 0;
		};

		// Read-only variable that cannot be modified by the sound instance,
		// and maybe shared amongst instances.
		struct FEnvironmentVariable
		{
			// Name to be displayed in editor or tools
			const FText DisplayName;

			// Description to be displayed in editor or tools
			const FText Description;

			// FName describing the type of the data.  May be
			// interpreted solely by a plugin or implementation
			// (ex. MetaSounds).  If blank, type is assumed to be
			// that described by ParamType.
			const FName DataType;

			// Name of variable used as a runtime identifier
			const FName ParamName;

			// Type of variable used as a runtime identifier if unspecified by the DataType.
			const EAudioParameterType ParamType = EAudioParameterType::None;
		};

		// Options used to restrict a corresponding UClass that interface may be applied to.
		struct FClassOptions
		{
			// Path to restricted UClass
			const FTopLevelAssetPath ClassPath;

			// Whether or not the class may be directly modifiable on an asset implementing
			// the given interface (added, removed, etc.)
			const bool bIsModifiable = true;

			// Whether or not the interface should be immediately added to the given class
			// type on creation.
			const bool bIsDefault = false;
		};

		FParameterInterface() = default;

		// Constructor used for parameter interface not limited to any particular UClass types
		AUDIOEXTENSIONS_API FParameterInterface(FName InName, const FVersion& InVersion);

		UE_DEPRECATED(5.3, "Set UClassOptions to determine what options apply for a given UClass (if any).")
		AUDIOEXTENSIONS_API FParameterInterface(FName InName, const FVersion& InVersion, const UClass& InClass);

		// Returns name of interface
		AUDIOEXTENSIONS_API FName GetName() const;

		// Returns version of interface
		AUDIOEXTENSIONS_API const FVersion& GetVersion() const;

		UE_DEPRECATED(5.3, "Use FParameterInterface::FindSupportedUClasses instead")
		AUDIOEXTENSIONS_API const UClass& GetType() const;

		AUDIOEXTENSIONS_API TArray<const UClass*> FindSupportedUClasses() const;

		// If specified, options used to restrict a corresponding UClass that interface may be
		// applied to.  If unspecified, interface is assumed to be applicable to any arbitrary UClass.
		AUDIOEXTENSIONS_API const TArray<FClassOptions>& GetUClassOptions() const;

		// Returns read-only array of inputs
		AUDIOEXTENSIONS_API const TArray<FInput>& GetInputs() const;

		// Returns read-only array of outputs
		AUDIOEXTENSIONS_API const TArray<FOutput>& GetOutputs() const;

		// Returns read-only array of environment variables
		AUDIOEXTENSIONS_API const TArray<FEnvironmentVariable>& GetEnvironment() const;

#if WITH_EDITORONLY_DATA
		// Description to be displayed in editor or tools
		AUDIOEXTENSIONS_API const FText& GetDescription() const;
#endif

	private:
		FName NamePrivate;
		FVersion VersionPrivate;

	protected:
		TArray<FInput> Inputs;
		TArray<FOutput> Outputs;
		TArray<FEnvironmentVariable> Environment;
		TArray<FClassOptions> UClassOptions;
		
#if WITH_EDITOR
		FText Description;
#endif
	};
	using FParameterInterfacePtr = TSharedPtr<FParameterInterface, ESPMode::ThreadSafe>;

	// Registry of engine-defined audio parameter interfaces, used to parameterize data provided
	// to or coming from an executable audio unit (ex. Sound like a MetaSoundSource, arbitrary DSP
	// graph like a MetaSoundPatch, etc.).
	class IAudioParameterInterfaceRegistry
	{
		static AUDIOEXTENSIONS_API TUniquePtr<IAudioParameterInterfaceRegistry> Instance;

	public:
		static AUDIOEXTENSIONS_API IAudioParameterInterfaceRegistry& Get();

		virtual ~IAudioParameterInterfaceRegistry() = default;

		// Iterate all registered interfaces
		virtual void IterateInterfaces(TFunction<void(FParameterInterfacePtr)> InFunction) const = 0;

		// Execute a given function when an interface is registered
		virtual void OnRegistration(TUniqueFunction<void(FParameterInterfacePtr)>&& InFunction) = 0;

		// Registers an interface
		virtual void RegisterInterface(FParameterInterfacePtr InInterface) = 0;

	protected:
		TSet<FParameterInterfacePtr> Interfaces;
		TUniqueFunction<void(FParameterInterfacePtr)> RegistrationFunction;
	};
} // namespace Audio
