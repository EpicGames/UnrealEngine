// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AudioParameter.h"
#include "AudioParameterControllerInterface.h"
#include "Containers/Array.h"
#include "CoreTypes.h"
#include "IAudioProxyInitializer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

// Forward Declarations
class UObject;
class FReferenceCollector;

namespace Audio
{
	using DeviceID = uint32;

	/** Data passed to CreateParameterTransmitter. */
	struct FParameterTransmitterInitParams
	{
		// Unique ID for this audio instance.
		uint64 InstanceID = uint64(INDEX_NONE);

		// Audio sample rate.
		float SampleRate = 0.0f;

		TArray<FAudioParameter> DefaultParams;

		// Audio device ID
		DeviceID AudioDeviceID = DeviceID(INDEX_NONE);

	};

	// Reference collector functionality for legacy parameter system
	// (i.e. backwards compatibility with the SoundCue system). None of this
	// should be used by future assets supporting parameters (ex. MetaSounds)
	// as object pointers within parameters should NOT be cached on threads
	// other than the GameThread, utilizing a proxy methodology like MetaSounds
	// that copies UObject data when and where necessary.
	class ILegacyParameterTransmitter
	{
		public:
			virtual ~ILegacyParameterTransmitter() = default;

			UE_DEPRECATED(5.6, "Use AddReferencedObjects(...) instead")
			AUDIOEXTENSIONS_API virtual TArray<const TObjectPtr<UObject>*> GetReferencedObjects() const;

			AUDIOEXTENSIONS_API virtual void AddReferencedObjects(FReferenceCollector& InCollector);
	};

	/** Interface for a audio instance transmitter.
	 *
	 * An audio instance transmitter ushers control parameters to a single audio object instance.
	 */
	class IParameterTransmitter : public ILegacyParameterTransmitter
	{
		public:
			static AUDIOEXTENSIONS_API const FName RouterName;

			virtual ~IParameterTransmitter() = default;

			// Reset parameters which stored on the transmitter.
			virtual void ResetParameters() {}

			// Called when the active sound is deleted due to the sound finishing,
			// being stopped, or being virtualized. 
			virtual void OnDeleteActiveSound() {}

			// Return the cached parameter with the given name if it exists
			// @return False if param not found, true if found.
			virtual bool GetParameter(FName InName, FAudioParameter& OutParam) const = 0;

			// Return reference to the cached parameter array.
			UE_DEPRECATED(5.6, "Access to internal parameters is no longer supported")
			virtual const TArray<FAudioParameter>& GetParameters() const = 0;

			virtual void CopyParameters(TArray<FAudioParameter>& OutParameters) const = 0;

			// Parameter Setters
			virtual bool SetParameters(TArray<FAudioParameter>&& InParameters) = 0;

			// Called when the active sound is virtualized  
			virtual void OnVirtualizeActiveSound() {}

			// Called when the virtualized active sound is realized 
			// @param Parameters to set 
			virtual void OnRealizeVirtualizedActiveSound(TArray<FAudioParameter>&& InParameters) {}
	};

	/** Base implementation for the parameter transmitter, which caches parameters
	  * and provides implementer to add additional logic to route parameter data accordingly.
	  */
	class FParameterTransmitterBase : public IParameterTransmitter
	{
	public:
		AUDIOEXTENSIONS_API FParameterTransmitterBase(TArray<FAudioParameter> InDefaultParams);
		AUDIOEXTENSIONS_API virtual ~FParameterTransmitterBase();

		AUDIOEXTENSIONS_API virtual bool GetParameter(FName InName, FAudioParameter& OutParam) const override;
		AUDIOEXTENSIONS_API virtual void ResetParameters() override;

		UE_DEPRECATED(5.6, "Access to internal parameter storage is no longer supported. Use CopyParameters(...) instead")
		AUDIOEXTENSIONS_API virtual const TArray<FAudioParameter>& GetParameters() const override;

		AUDIOEXTENSIONS_API virtual void CopyParameters(TArray<FAudioParameter>& OutParameters) const override;

		AUDIOEXTENSIONS_API virtual bool SetParameters(TArray<FAudioParameter>&& InParameters) override;

		AUDIOEXTENSIONS_API virtual void AddReferencedObjects(FReferenceCollector& InCollector) override;

		AUDIOEXTENSIONS_API virtual void OnVirtualizeActiveSound() override;
		AUDIOEXTENSIONS_API virtual void OnRealizeVirtualizedActiveSound(TArray<FAudioParameter>&& InParameters);
		AUDIOEXTENSIONS_API bool IsVirtualized() const;

	protected:

		AUDIOEXTENSIONS_API void AddReferencedObjectsFromParameters(FReferenceCollector& InCollector, TArrayView<FAudioParameter> InParameters) const;

		UE_DEPRECATED(5.6, "Subclasses of FParameterTransmitterBase should not access AudioParameters member directly. These will be moved to private visibility soon.")
		TArray<FAudioParameter> AudioParameters;
		UE_DEPRECATED(5.6, "Subclasses of FParameterTransmitterBase should not access bIsVirtualized member directly. These will be moved to private visibility soon.")
		bool bIsVirtualized;

	};
} // namespace Audio
