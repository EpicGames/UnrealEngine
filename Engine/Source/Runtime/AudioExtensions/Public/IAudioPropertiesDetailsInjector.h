// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Features/IModularFeature.h"
#include "Templates/SharedPointer.h"

#define UE_API AUDIOEXTENSIONS_API

class IDetailLayoutBuilder;
class IPropertyHandle;

namespace AudioPropertiesDetailsInjector
{
	static const FLazyName BuilderModularFeatureName = TEXT("AudioPropertiesDetailsInjectorBuilder");
	
}

class IAudioPropertiesDetailsInjector : public TSharedFromThis<IAudioPropertiesDetailsInjector>
{
public:
	UE_API virtual ~IAudioPropertiesDetailsInjector();

	virtual void CustomizeInjectedPropertiesDetails(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle) = 0;
	virtual void BindDetailCustomizationToPropertySheetChanges(IDetailLayoutBuilder& DetailBuilder, TSharedRef<IPropertyHandle> PropertySheetPropertyHandle) = 0;
	virtual void UnbindFromPropertySheetChanges() = 0;
};

class IAudioPropertiesDetailsInjectorBuilder : public TSharedFromThis<IAudioPropertiesDetailsInjectorBuilder>, public IModularFeature
{
public:
	UE_API virtual ~IAudioPropertiesDetailsInjectorBuilder();

    virtual IAudioPropertiesDetailsInjector* CreateAudioPropertiesDetailsInjector() = 0;
};

#undef UE_API
