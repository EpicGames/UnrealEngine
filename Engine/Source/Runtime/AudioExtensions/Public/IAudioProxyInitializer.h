// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioUnknown.h"
#include "Misc/AssertionMacros.h"
#include "Misc/CoreMiscDefines.h"
#include "Misc/ScopeTryLock.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

#include <type_traits>


template <typename T>
class TDataSnapshotProducer;

template <typename T>
class TDataSnapshot
{
public:
	TDataSnapshot() = delete;
	
	TDataSnapshot(TWeakPtr<const TDataSnapshotProducer<T>> InWeakProducer, TSharedPtr<T> InPublishedData)
	: WeakProducer(InWeakProducer)
	, DataSnapshot(InPublishedData)
	{}

	operator TSharedPtr<T>() const
	{
		return DataSnapshot;
	}

	bool IsValid() const
	{
		return DataSnapshot.IsValid();
	}

	explicit operator bool() const
	{
		return DataSnapshot.IsValid();
	}
	
	T* operator->() const
	{
		return DataSnapshot.Get();
	}

	bool IsUpdateAvailable() const
	{
		if (TSharedPtr<const TDataSnapshotProducer<T>> PinnedProducer = WeakProducer.Pin())
		{
			return PinnedProducer->IsSnapshotUpdateAvailable(DataSnapshot);
		}
		
		return false;
	}

	bool GetLatest()
	{
		TSharedPtr<const TDataSnapshotProducer<T>> PinnedProducer = WeakProducer.Pin();

		if (!PinnedProducer)
		{
			// producer is out of scope, no more updates
			return false;
		}

		if (PinnedProducer->IsSnapshotUpdateAvailable(DataSnapshot))
		{
			// grab new snapshot
			DataSnapshot = PinnedProducer->GetData();
		
			return true;
		}
		
		return false;
	}
	
private:
	TWeakPtr<const TDataSnapshotProducer<T>> WeakProducer;
	TSharedPtr<T> DataSnapshot;
};

template <typename T>
class TDataSnapshotProducer : public TSharedFromThis<TDataSnapshotProducer<T>>
{
public:
	operator TSharedPtr<T>() const
	{
		return WorkingData;
	}

	bool IsValid() const
	{
		return WorkingData.IsValid();
	}
	
	explicit operator bool() const
	{
		return WorkingData.IsValid();
	}
	
	T* operator->() const
	{
		return WorkingData.Get();
	}

	void Fork()
	{
		FWriteScopeLock Lock(DataLock);
		WorkingData = MakeShared<T>();
	}

	// note: there are still some things the USoundWave will potentially update
	// on existing proxies (i.e. certain flags). Letting these pointers alias 
	// after "Publish()" is intentional for now.
	void Publish()
	{
		FWriteScopeLock Lock(DataLock);
		PublishedData = WorkingData;
	}
	
	TDataSnapshot<T> GetSnapshot() const
	{
		return TDataSnapshot<T>(this->AsWeak(), GetData());
	}

	bool IsSnapshotUpdateAvailable(const TSharedPtr<T>& InSnapshot) const
	{
		if (DataLock.TryReadLock())
		{
			const bool bIsUpdateAvailable = PublishedData.IsValid() && (PublishedData != InSnapshot);

			DataLock.ReadUnlock();
			return bIsUpdateAvailable;
		}

		return false;
	}
	
private:
	TSharedPtr<T> GetData() const
	{
		FReadScopeLock Lock(DataLock);
		return PublishedData;
	}
	
	TSharedPtr<T> PublishedData;
	TSharedPtr<T> WorkingData = MakeShared<T>();
	mutable FRWLock DataLock;
	
	friend class TDataSnapshot<T>;
}; // TDataSnapshotProducer


/**
 * Interfaces for Audio Proxy Objects 
 * These are used to spawn thread safe instances of UObjects that may be garbage collected on the game thread.
 * In shipping builds, these are effectively abstract pointers, but CHECK_AUDIOPROXY_TYPES can optionally be used
 * to check downcasts.
 */

#define  IMPL_AUDIOPROXY_CLASS(FClassName) \
	static FName GetAudioProxyTypeName() \
	{ \
		static FName MyClassName = #FClassName; \
		return MyClassName; \
	} \
	static FName GetInterfaceId() { return GetAudioProxyTypeName(); } \
	virtual void* QueryInterface(const FName InterfaceId) override \
	{ \
		if (InterfaceId == GetAudioProxyTypeName()) \
		{ \
			return this; \
		} \
		return IProxyData::QueryInterface(InterfaceId); \
	} \
	static constexpr bool bWasAudioProxyClassImplemented = true; \
	friend class ::Audio::IProxyData; \
	friend class ::Audio::TProxyData<FClassName>;


namespace Audio
{
	// Forward Declarations
	class IProxyData;
	using IProxyDataPtr UE_DEPRECATED(5.2, "Replace IProxyDataPtr with TSharedPtr<Audio::IProxyData>") = TUniquePtr<Audio::IProxyData>;

	/*
	 * Base class that allows us to typecheck proxy data before downcasting it in debug builds.
	*/
	class IProxyData : public IUnknown
	{
	private:
		FName ProxyTypeName;
	public:
		virtual ~IProxyData() = default;

		// IUnknown.
		using IUnknown::QueryInterface;
		virtual void* QueryInterface(const FName InterfaceId) override 
		{
			return nullptr;	
		}
					
		FName GetProxyTypeName() const
		{
			return ProxyTypeName;
		}

		template<typename ProxyType>
		ProxyType& GetAs()
		{
			static_assert(std::is_base_of_v<IProxyData, ProxyType>, "Tried to downcast IProxyInitData to an unrelated type!");
			ProxyType* Result = static_cast<ProxyType*>(QueryInterface(ProxyType::GetAudioProxyTypeName()));
			checkf(Result, TEXT("Tried to downcast type %s to %s!"), *GetProxyTypeName().ToString(), *ProxyType::GetAudioProxyTypeName().ToString());
			return *Result;
		}

		template<typename ProxyType>
		const ProxyType& GetAs() const
		{
			static_assert(std::is_base_of_v<IProxyData, ProxyType>, "Tried to downcast IProxyInitData to an unrelated type!");
			const ProxyType* Result = static_cast<const ProxyType*>(QueryInterface(ProxyType::GetAudioProxyTypeName()));
			checkf(Result, TEXT("Tried to downcast type %s to %s!"), *GetProxyTypeName().ToString(), *ProxyType::GetAudioProxyTypeName().ToString());
			return *Result;
		}

		template<typename ProxyType>
		bool CheckTypeCast() const
		{
			return QueryInterface(ProxyType::GetAudioProxyTypeName()) != nullptr;
		}

		IProxyData(FName InProxyTypeName)
			: ProxyTypeName(InProxyTypeName)
		{}

		UE_DEPRECATED(5.2, "Proxy data is stored in a TSharedPtr<> and no longer requires cloning")
		virtual TUniquePtr<IProxyData> Clone() const { return nullptr; }
	};

	/**
	 * This class can be implemented to create a custom, threadsafe instance of a given UObject.
	 * This is a CRTP class, and should always be subclassed with the name of the subclass.
	 */
	template <typename Type>
	class TProxyData : public IProxyData
	{
	protected:
		static constexpr bool bWasAudioProxyClassImplemented = false;

	public:
		TProxyData()
			: IProxyData(Type::GetAudioProxyTypeName())
		{
			static_assert(Type::bWasAudioProxyClassImplemented, "Make sure to include IMPL_AUDIOPROXY_CLASS(ClassName) in your implementation of TProxyData.");
		}
	};

	struct FProxyDataInitParams
	{
		FName NameOfFeatureRequestingProxy;
	};
} // namespace Audio

/*
* This can be subclassed to make a UClass an audio proxy factory.
*/
class IAudioProxyDataFactory
{
public:
	UE_DEPRECATED(5.2, "Call TSharedPtr<Audio::IProxyData> CreateProxyData(...) instead of a TUniquePtr<Audio::IProxyData> CreateNewProxyData(...).")
	AUDIOEXTENSIONS_API virtual TUniquePtr<Audio::IProxyData> CreateNewProxyData(const Audio::FProxyDataInitParams& InitParams);

	AUDIOEXTENSIONS_API virtual TSharedPtr<Audio::IProxyData> CreateProxyData(const Audio::FProxyDataInitParams& InitParams);
};

namespace Audio
{
	template <typename UClassToUse>
	IAudioProxyDataFactory* CastToProxyDataFactory(UObject* InObject)
	{
		if constexpr (std::is_base_of_v<IAudioProxyDataFactory, UClassToUse>)
		{
			if (InObject)
			{
				UClassToUse* DowncastObject = Cast<UClassToUse>(InObject);
				if (ensureAlways(DowncastObject))
				{
					return static_cast<IAudioProxyDataFactory*>(DowncastObject);
				}
			}
		}

		return nullptr;
	}
} // namespace Audio
