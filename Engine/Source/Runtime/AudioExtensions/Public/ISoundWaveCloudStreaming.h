// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Features/IModularFeature.h"
#include "Misc/Guid.h"

#include "ISoundWaveCloudStreaming.generated.h"

//
// Forward declarations
//
class USoundWave;
class IDetailLayoutBuilder;


namespace Audio
{

	class ISoundWaveCloudStreamingFeature : public IModularFeature
	{
	public:
		static FName GetModularFeatureName() { return TEXT("SoundWaveCloudStreaming"); }
		virtual ~ISoundWaveCloudStreamingFeature() = default;

		/** Returns the GUID of this plugin. */
		virtual FGuid GetPluginGUID() const = 0;

		/** Checks if the given sound wave can be turned into cloud streamable. */
		virtual bool CanOverrideFormat(const USoundWave* InWaveToOverride) = 0;
		
		/** Gets the format name to use when overriding the given sound wave for cloud streaming. */
		virtual FName GetOverrideFormatName(const USoundWave* InWaveToOverride) = 0;
#if WITH_EDITOR
		/** Gets a hash of the parameters for the DDC. */
		virtual FString GetOverrideParameterDDCHash(const USoundWave* InWaveToOverride) = 0;

		/** Add editor customization for an instance. */
		virtual bool AddCustomizationCloudStreamingPlatformDetails(IDetailLayoutBuilder& InDetailLayoutBuilder) = 0;
#endif // WITH_EDITOR
	};

}


/** Platform specific enabling of Sound Wave cloud streaming. */
UENUM()
enum class ESoundWaveCloudStreamingPlatformProjectEnableType : uint8
{
	/** Enabled for this platform. */
	Enabled,

	/** Disabled for this platform. */
	Disabled,
};

/** Platform specific settings for Sound Wave cloud streaming. */
USTRUCT()
struct FSoundWaveCloudStreamingPlatformProjectSettings
{
	GENERATED_USTRUCT_BODY()

	FSoundWaveCloudStreamingPlatformProjectSettings()
	{
		EnablementSetting = ESoundWaveCloudStreamingPlatformProjectEnableType::Disabled;
	}

	bool IsDefault() const
	{
		return EnablementSetting == ESoundWaveCloudStreamingPlatformProjectEnableType::Disabled;
	}

	/** Overrides whether to use cloud streaming on this platform. */
	UPROPERTY(EditAnywhere, Category = Platforms)
	ESoundWaveCloudStreamingPlatformProjectEnableType EnablementSetting;
};



/** Platform specific enabling of Sound Wave cloud streaming. */
UENUM()
enum class ESoundWaveCloudStreamingPlatformEnableType : uint8
{
	/** Use Sound Wave setting. */
	Inherited,

	/** Disables Sound Wave cloud streaming for this platform. */
	Disabled,

	/** Used in Slate widget configuration to indicate multiple selected objects have different values. */
	SWC_MultipleValues UMETA(Hidden, DisplayName="Multiple values")
};

/** Platform specific settings for Sound Wave cloud streaming. */
USTRUCT()
struct FSoundWaveCloudStreamingPlatformSettings
{
	GENERATED_USTRUCT_BODY()

	FSoundWaveCloudStreamingPlatformSettings()
	{
		EnablementSetting = ESoundWaveCloudStreamingPlatformEnableType::Inherited;
	}

	bool IsDefault() const
	{
		return EnablementSetting == ESoundWaveCloudStreamingPlatformEnableType::Inherited;
	}

	/** Overrides whether to use cloud streaming on this platform. */
	UPROPERTY(EditAnywhere, Category = Platforms)
	ESoundWaveCloudStreamingPlatformEnableType EnablementSetting;
};
