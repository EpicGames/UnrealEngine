// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixerNullDevice.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "DSP/MultithreadedPatching.h"
#include "Features/IModularFeatures.h"
#include "HAL/CriticalSection.h"
#include "HAL/ThreadSafeCounter.h"
#include "ISoundfieldFormat.h"
#include "Misc/AssertionMacros.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ISoundfieldEndpoint.generated.h"

class UClass;
struct FAudioPluginInitializationParams;
template <typename FuncType> class TFunctionRef;


/**
 * Interfaces for Soundfield Endpoints
 * 
 * This set of interfaces can be used to 
 * 
 */

/**  
 * This interface should be used to provide a non-uclass version of the data described in
 * your implementation of USoundfieldEndpointSettingsBase.
 */
class ISoundfieldEndpointSettingsProxy
{
public:
	virtual ~ISoundfieldEndpointSettingsProxy() = default;
};

/**
 * This opaque class should be used for specifying settings for how audio should be
 * send to an external endpoint.
 */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundfieldEndpointSettingsBase : public UObject
{
	GENERATED_BODY()

public:
	virtual TUniquePtr<ISoundfieldEndpointSettingsProxy> GetProxy() const PURE_VIRTUAL(USoundfieldEndpointSettingsBase::GetProxy, return nullptr;);
};

/**
 * Class that allows soundfield-encoded audio to be sent to an arbitrary locale.
 * For endpoint types that support receiving our downmixed interleaved audio data directly, implement IAudioEndpoint instead.
 */
class ISoundfieldEndpoint
{
public:
	/**
	 * @param [NumRenderCallbacksToBuffer] Maximum number of ISoundfieldPackets that should be buffered.
	 */
	AUDIOEXTENSIONS_API ISoundfieldEndpoint(int32 NumRenderCallbacksToBuffer);

	virtual ~ISoundfieldEndpoint() = default;

	/**
	 * Push a soundfield packet to the buffer.
	 */
	AUDIOEXTENSIONS_API bool PushAudio(TUniquePtr<ISoundfieldAudioPacket>&& InPacket);

	/**
	 * Post new settings for this endpoint.
	 * There is no type safety on this call, so make sure that you are using the correct implementation
	 * of IAudioEndpointSettingsProxy for this implementation of IAudioEndpoint.
	 */
	AUDIOEXTENSIONS_API void SetNewSettings(TUniquePtr<ISoundfieldEndpointSettingsProxy>&& InNewSettings);

	// Returns the amount of ISoundfieldAudioPackets currently buffered for this endpoint.
	AUDIOEXTENSIONS_API int32 GetNumPacketsBuffer();

	// Returns the amount of ISoundfieldAudioPackets that can be buffered for this endpoint before reaching capacity.
	AUDIOEXTENSIONS_API int32 GetRemainderInPacketBuffer();

	/**
	 * If this endpoint hasn't created it's own async callback thread but still requires an explicit callback, this should be called.
	 */
	AUDIOEXTENSIONS_API void ProcessAudioIfNecessary();

protected:

	/** OPTIONAL OVERRIDES: */

	/**
	 * For endpoints that do not explicitly fire a timed callback to poll audio data,
	 * this should be overridden to return true, and  OnAudioCallback and GetDesiredCallbackFrequency should be overridden.
	 */
	virtual bool EndpointRequiresCallback() const { return false; }

	/**
	 * For endpoints that return true for EndpointRequiresCallback, this should return the duration between OnAudioCallback calls in seconds.
	 */
	virtual float GetDesiredCallbackPeriodicity() const { return 0; }

	/**
	 * For endpoints that override EndpointRequiresCallback to return true, 
	 * this callback will be called every (GetDesiredNumFrames() / GetSampleRate()) seconds.
	 * @param [in] InPacket the next buffer of audio. Can be nullptr if PushAudio hasn't been called in a while and the buffer is starved.
	 * @param [in] InSettings is the most recent soundfield settings for this endpoint. Can be null.
	 */
	virtual void OnAudioCallback(TUniquePtr<ISoundfieldAudioPacket>&& InPacket, const ISoundfieldEndpointSettingsProxy* InSettings) { return; };

	/** METHODS USED BY IMPLEMENTATIONS OF ISoundfieldEndpoint: */

	/**
	 * This is used by the IAudioEndpoint implementation to poll buffered audio to process or send to the endpoint.
	 * Note that this is NOT thread safe if EndpointRequiresCallback() is overridden to return true. If that is the case, use an override of OnAudioCallback instead.
	 * 
	 * 
	 */
	AUDIOEXTENSIONS_API TUniquePtr<ISoundfieldAudioPacket> PopAudio();

	/**
	 * Use this as a thread safe way to use the current settings posted to this IAudioEndpoint. Locks with IAudioEndpoint::SetSettings.
	 * @param[in] NewSettingsRetrieved lambda used to work with the retrieved settings.
	 *                                 This lambda is called immediately and synchronously, but is used
	 *                                 to safely scope usage of the IAudioEndpointSettingsProxy pointer.
	 *                                 Note that the resulting ISoundfieldEndpointSettingsProxy can be null.
	 */
	AUDIOEXTENSIONS_API void PollSettings(TFunctionRef<void(const ISoundfieldEndpointSettingsProxy*)> SettingsCallback);

	/**
	 * If EndpointRequiresCallback() returns true, this will be used to spawn an async thread and begin calling OnAudioCallback.
	 */
	AUDIOEXTENSIONS_API void StartRunningCallback();
	AUDIOEXTENSIONS_API void StopRunningCallback();

	AUDIOEXTENSIONS_API void RunCallbackSynchronously();

private:
	// Owns a scoped thread and runs OnAudioCallback when StartRunningCallback() is called.
	TUniquePtr<Audio::FMixerNullCallback> RenderCallback;
	
	// This array is used as a non-copying circular buffer for ISoundfieldAudioPackets.
	TArray<TUniquePtr<ISoundfieldAudioPacket>> AudioPacketBuffer;
	FThreadSafeCounter ReadCounter;
	FThreadSafeCounter WriteCounter;

	// Owns the current settings for this endpoint.
	TUniquePtr<ISoundfieldEndpointSettingsProxy> CurrentSettings;
	FCriticalSection CurrentSettingsCriticalSection;
};

/**
 * This factory is used to expose Soundfield Endpoint types to the editor.
 * Once a factory is constructed and RegisterEndpointType is called, it will be exposed as a type of endpoint
 * That a submix in the submix graph could be constructed with.
 * Also note that an implementation of ISoundfieldDecoder is not necessary for soundfield formats that are only used for
 * soundfield endpoints.
 */
class ISoundfieldEndpointFactory : public ISoundfieldFactory
{
public:
	/** Virtual destructor */
	virtual ~ISoundfieldEndpointFactory() = default;

	/** Get the name for the endpoint type that this factory produces.  */
	virtual FName GetEndpointTypeName() = 0;

	/** This is the FName used to register Soundfield Endpoint factories with the modular feature system. */
	static FName GetModularFeatureName()
	{
		static FName SoundfieldEndpointName = FName("Soundfield Endpoint");
		return SoundfieldEndpointName;
	}

	/** 
	 * This needs to be called to make a soundfield format usable by the engine.
	 * It can be called from a ISoundfieldFactory subclass' constructor
	*/
	static void RegisterEndpointType(ISoundfieldEndpointFactory* InFactory)
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), InFactory);
	}

	/**
	 * This needs to be called it an implementation of ISoundfieldFactory is about to be destroyed.
	 * It can be called from the destructor of an implementation of ISoundfieldFactory.
	 */
	static void UnregisterEndpointType(ISoundfieldEndpointFactory* InFactory)
	{
		IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), InFactory);
	}

	/**
	 * Get a registered endpoint factory by name.
	 */
	static AUDIOEXTENSIONS_API ISoundfieldEndpointFactory* Get(const FName& InName);

	static AUDIOEXTENSIONS_API TArray<FName> GetAllSoundfieldEndpointTypes();

	/**
	 * This is the default name used when a user creates a soundfield endpoint submix.
	 * Soundfied Endpoint submixes with this type will send their audio to the default output
	 * with no encoding.
	 */
	static AUDIOEXTENSIONS_API FName DefaultSoundfieldEndpointName();

	/** This function is not necessary to override, since audio sent to an endpoint does not need to be decoded to interleaved audio buffers. */
	AUDIOEXTENSIONS_API TUniquePtr<ISoundfieldDecoderStream> CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) override;

	/** REQUIRED OVERRIDES: */

	/**
	 * These overrides are required from ISoundfieldFactory:
	 * ISoundfieldFactory::CreateNewEncoderStream
	 * ISoundfieldFactory::CreateNewTranscoderStream
	 * ISoundfieldFactory::CreateNewMixerStream
	 * ISoundfieldFactory::CreateEmptyPacket
	 * ISoundfieldFactory::CanTranscodeFromSoundfieldFormat
	 * ISoundfieldFactory::GetCustomEncodingSettingsClass
	 * ISoundfieldFactory::GetDefaultEncodingSettings
	 */

	/** Called for every new endpoint submix created with this factory's endpoint type. */
	virtual TUniquePtr<ISoundfieldEndpoint> CreateNewEndpointInstance(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEndpointSettingsProxy& InitialSettings) = 0;


	/**
	 * Should return the StaticClass of this factory's implementation of USoundfieldEndpointSettingsBase.
	 */
	virtual UClass* GetCustomEndpointSettingsClass() const
	{
		return nullptr;
	}

	/**
	 * return the settings an endpoint should use if a soundfield endpoint submix did not have their settings specified.
	 */
	virtual USoundfieldEndpointSettingsBase* GetDefaultEndpointSettings() = 0;

	bool IsEndpointFormat() override { return true; }

	AUDIOEXTENSIONS_API virtual FName GetSoundfieldFormatName() override;
	AUDIOEXTENSIONS_API virtual bool CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings) override;
};
