// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "Math/Quat.h"
#include "Misc/AssertionMacros.h"
#include "Templates/TypeHash.h"
#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "ISoundfieldFormat.generated.h"

class FArchive;
class UClass;


/**
 * Interfaces for Soundfield Encoding and Decoding
 * 
 * This set of interfaces can be implemented to add support for encoding to and decoding from a soundfield format.
 * There are four classes that are required to implement a soundfield format:
 *
 * ISoundfieldPacket: Container for a single audio callback's resulting data for the soundfield format.
 *                    For example, for first order ambisonics, this would contain an interleaved buffer of floats.
 *                    For a proprietary spatial audio format, this would contain a bitstream.
 *
 * USoundfieldEncodingSettingsBase: These are game thread-local, inspectable settings to define how a soundfield is getting encoded.
 *                                  For example, this may contain the order of an ambisonics sound field.
 *
 * ISoundfieldEncodingSettingsProxy: This should contain all data from USoundfieldEncodingSettingsBase that needs to be read by an
 *                                   encoder or transcoder stream to apply the settings from an implementation of USoundfieldEncodingSettingsBase.
 * 
 * 
 * ISoundfieldEncoderStream: Class that encodes interleaved audio from a set of arbitrary locations to an ISoundfieldPacket.
 * ISoundfieldDecoderStream: Class that decodes audio from an ISoundfieldPacket to interleaved audio at a set of arbitrary locations.
 * ISoundfieldTranscodeStream: Class that translates a soundfield stream from one type to another.
 * ISoundfieldMixerStream: Class that sums together multiple incoming ISoundfieldPackets into one ISoundfieldPacket.
 *
 * ISoundfieldFactory: Factory class that declares the name of your format and creates new encoder, decoders, transcoders and mixers as requested.
 * 
 */

// List of classes that that interface with Soundfield objects:
namespace Audio
{
	class FMixerDevice;
	class FMixerSourceManager;
	class FMixerSubmix;

	struct FChannelPositionInfo
	{
		EAudioMixerChannel::Type Channel = EAudioMixerChannel::DefaultChannel;

		// Horizontal angle of the position of this channel, in radians.
		// Increases as the channel moves clockwise about the listener.
		// Goes from -PI to PI.
		float Azimuth = 0.0f;
		// Vertical angle of the position of this channel, in radians.
		// Increases as the height of the channel increases proportional to the channel's distance from the listener.
		// Goes from -PI to PI.
		float Elevation = 0.0f;

		// distance from the listener. By default, channels are typically on the unit sphere and have a radius of 1.0f.
		// For spatialized inputs, this radius will be expressed in Unreal Units.
		float Radius = 1.0f;
	};
}

/** 
 * This helper function is used to downcast abstract objects during callbacks.
 * Since implementing this API requires frequent downcasting of opaque data, and RTTI is not
 * enabled by default in our codebase, This is useful for avoiding programmer error.
 */
template<typename ToType, typename FromType>
ToType& DowncastSoundfieldRef(FromType& InRef)
{
#if PLATFORM_WINDOWS
	constexpr bool bIsToTypeChildClass = std::is_base_of<FromType, ToType>::value;
	static_assert(bIsToTypeChildClass, "Tried to cast a reference to an unrelated type.");
#endif

	check(&InRef != nullptr);

	return *static_cast<ToType*>(&InRef);
}


/**
 * This interface should be used to provide a non-uclass version of the data described in
 * your implementation of USoundfieldEncodingSettingsBase. We will then pass this proxy 
 * object to the soundfield stream classes.
 */
class ISoundfieldEncodingSettingsProxy
{
public:
	virtual ~ISoundfieldEncodingSettingsProxy() = default;

	/** 
	 * This should return a unique 
	 * This is used so that we don't call the same encode operation multiple times for a single source being sent to identical submixes. 
	 */
	virtual uint32 GetUniqueId() const = 0;

	/** This should return a new, identical encoding settings. */
	virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> Duplicate() const = 0;
};

/** 
 * This opaque class should be used for specifying settings for how audio should be encoded
 * to your soundfield format for a given submix or file.
 */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundfieldEncodingSettingsBase : public UObject
{
	GENERATED_BODY()

public:
	virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> GetProxy() const PURE_VIRTUAL(USoundfieldEncodingSettingsBase::GetProxy, return nullptr;);
};

struct FAudioPluginInitializationParams;

/**
 * This interface represents all encoded soundfield audio from a single render callback.
 */
class ISoundfieldAudioPacket
{
public:
	virtual ~ISoundfieldAudioPacket() = default;

	/**
	 * Read or write this packet to a byte buffer.
	 */
	virtual void Serialize(FArchive& Ar) = 0;

	/**
	 * Create a new version of this packet.
	 */
	virtual TUniquePtr<ISoundfieldAudioPacket> Duplicate() const = 0;

	/**
	 * Zero out the contents of this packet.
	 */
	virtual void Reset() = 0;
};

/**
 * Positional data for each channel.
 */
struct FSoundfieldSpeakerPositionalData
{
	int32 NumChannels = 0;

	const TArray<Audio::FChannelPositionInfo>* ChannelPositions = nullptr;

	// For encoding, this is the rotation of the emitter source relative to the world.
	// For decoding, this is the rotation of the listener relative to the output speaker bed.
	FQuat Rotation = FQuat::Identity;
};

/**
 * All input parameters for a single Encode operation.
 */
struct FSoundfieldEncoderInputData
{
	/*
	 * Input buffer of interleaved floats. Each channel of the interleaved AudioBuffer corresponds to a channel index in PositionalData.
	 */
	Audio::FAlignedFloatBuffer& AudioBuffer;

	// Number of channels of the source audio buffer.
	int32 NumChannels;

	// if the input audio was already encoded to ambisonics,
	// this will point to the settings the audio was encoded with.
	// Otherwise, this will be a null pointer.
	ISoundfieldEncodingSettingsProxy& InputSettings;

	FSoundfieldSpeakerPositionalData& PositionalData;
};

class ISoundfieldEncoderStream
{
public:
	virtual ~ISoundfieldEncoderStream() = default;

	virtual void Encode(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) = 0;
	virtual void EncodeAndMixIn(const FSoundfieldEncoderInputData& InputData, ISoundfieldAudioPacket& OutputData) = 0;
};

struct FSoundfieldDecoderInputData
{
	ISoundfieldAudioPacket& SoundfieldBuffer;

	// The positions of the channels we will output FSoundfieldDecoderOutputData::AudioBuffer to.
	FSoundfieldSpeakerPositionalData& PositionalData;

	int32 NumFrames;
	float SampleRate;
};

struct FSoundfieldDecoderOutputData
{
	Audio::FAlignedFloatBuffer& AudioBuffer;
};

class ISoundfieldDecoderStream
{
public:
	virtual ~ISoundfieldDecoderStream() = default;

	virtual void Decode(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) = 0;
	virtual void DecodeAndMixIn(const FSoundfieldDecoderInputData& InputData, FSoundfieldDecoderOutputData& OutputData) = 0;
};

class ISoundfieldTranscodeStream
{
public:
	virtual ~ISoundfieldTranscodeStream() = default;

	virtual void Transcode(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& OutputData, const ISoundfieldEncodingSettingsProxy& OutputSettings) = 0;
	virtual void TranscodeAndMixIn(const ISoundfieldAudioPacket& InputData, const ISoundfieldEncodingSettingsProxy& InputSettings, ISoundfieldAudioPacket& PacketToMixTo, const ISoundfieldEncodingSettingsProxy& OutputSettings) = 0;
};

struct FSoundfieldMixerInputData
{
	// Packet that should be mixed into the output.
	const ISoundfieldAudioPacket& InputPacket;
	// settings used to encode both the input packet than the packet we are mixing to.
	const ISoundfieldEncodingSettingsProxy& EncodingSettings;
	// The amount, in linear gain, to
	float SendLevel;
};

class ISoundfieldMixerStream
{
public:
	virtual ~ISoundfieldMixerStream() = default;

	virtual void MixTogether(const FSoundfieldMixerInputData& InputData, ISoundfieldAudioPacket& PacketToMixInto) = 0;
};

class ISoundfieldFactory : public IModularFeature
{
public:
	/** Virtual destructor */
	virtual ~ISoundfieldFactory() = default;

	/** When a submix has this format name, it is using interleaved, floating point audio with no metadata. */
	static AUDIOEXTENSIONS_API FName GetFormatNameForNoEncoding();

	/** When a submix has this format name, it derives its format from the submix it sends audio to. */
	static AUDIOEXTENSIONS_API FName GetFormatNameForInheritedEncoding();

	/** This is the FName used to register Soundfield Format factories with the modular feature system. */
	static AUDIOEXTENSIONS_API FName GetModularFeatureName();

	/** 
	 * This needs to be called to make a soundfield format usable by the engine.
	 * It can be called from a ISoundfieldFactory subclass' constructor
	*/
	static AUDIOEXTENSIONS_API void RegisterSoundfieldFormat(ISoundfieldFactory* InFactory);

	/**
	 * This needs to be called it an implementation of ISoundfieldFactory is about to be destroyed.
	 * It can be called from the destructor of an implementation of ISoundfieldFactory.
	 */
	static AUDIOEXTENSIONS_API void UnregisterSoundfieldFormat(ISoundfieldFactory* InFactory);

	/**
	 * Get a registered soundfield format factory by name.
	 */
	static AUDIOEXTENSIONS_API ISoundfieldFactory* Get(const FName& InName);

	static AUDIOEXTENSIONS_API TArray<FName> GetAvailableSoundfieldFormats();

	/** Get soundfield format name  */
	virtual FName GetSoundfieldFormatName() = 0;

	/** Called when a stream is opened. */
	virtual TUniquePtr<ISoundfieldEncoderStream> CreateEncoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) = 0;
	virtual TUniquePtr<ISoundfieldDecoderStream> CreateDecoderStream(const FAudioPluginInitializationParams& InitInfo, const ISoundfieldEncodingSettingsProxy& InitialSettings) = 0;

	// Transcoder streams are fed a soundfield audio packet with either a different format entirely, or the same format and different settings.
	// Specifying and returns a Transcoder Stream is not necessary if CanTranscodeSoundfieldFormat and ShouldReencodeBetween always returns false.
	virtual TUniquePtr<ISoundfieldTranscodeStream> CreateTranscoderStream(const FName SourceFormat, const ISoundfieldEncodingSettingsProxy& InitialSourceSettings, const FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& InitialDestinationSettings, const FAudioPluginInitializationParams& InitInfo) = 0;
	virtual TUniquePtr<ISoundfieldMixerStream> CreateMixerStream(const ISoundfieldEncodingSettingsProxy& InitialSettings) = 0;
	virtual TUniquePtr<ISoundfieldAudioPacket> CreateEmptyPacket() = 0;

	/*
	* Override this function to determine whether an incoming ISoundfieldPacket would need to be explicitly operated on between two submixes with the same format, but potentially different encoding settings.
	* If this returns true, a new transcoder will be created.
	* if this returns false, then the source submix's ISoundfieldPacket will be passed down directly.
	*/
	virtual bool IsTranscodeRequiredBetweenSettings(const ISoundfieldEncodingSettingsProxy& SourceSettings, const ISoundfieldEncodingSettingsProxy& DestinationSettings)
	{
		return true;
	}

	/**
	 * Override this function to decide whether this soundfield format can read and convert from a source format.
	 */
	virtual bool CanTranscodeFromSoundfieldFormat(FName SourceFormat, const ISoundfieldEncodingSettingsProxy& SourceEncodingSettings) = 0;
	virtual bool CanTranscodeToSoundfieldFormat(FName DestinationFormat, const ISoundfieldEncodingSettingsProxy& DestinationEncodingSettings) = 0;

	/**
	 * If this is overridden to true, we will set up a separate encoding stream for every submix plugged into this soundfield submix.
	 * Otherwise, we mix all non-soundfield submixes plugged into this soundfield submix together and use one encoding stream.
	 */
	virtual bool ShouldEncodeAllStreamsIndependently(const ISoundfieldEncodingSettingsProxy& EncodingSettings)
	{
		return false;
	}

	/**
	 * Should return the StaticClass of your implementation of USoundfieldEncodingSettingsBase.
	 */
	virtual UClass* GetCustomEncodingSettingsClass() const
	{
		return nullptr;
	}

	virtual const USoundfieldEncodingSettingsBase* GetDefaultEncodingSettings() = 0;

	/** 
	 * This is overridden to return true for soundfield formats that are only used for sending audio externally.
	 * Rather than overriding this, consider implementing ISoundfieldEndpointFactory.
	*/
	virtual bool IsEndpointFormat() { return false; }
};

class ISoundfieldEffectSettingsProxy
{
public:
	virtual ~ISoundfieldEffectSettingsProxy() = default;
};

UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundfieldEffectSettingsBase : public UObject
{
	GENERATED_BODY()

protected:
	virtual TUniquePtr<ISoundfieldEffectSettingsProxy> GetNewProxy() const PURE_VIRTUAL(USoundfieldEffectSettingsBase::GetProxy, return nullptr;);

private:
	// This is called by any engine system that is explicitly marked as a friend.
	TUniquePtr<ISoundfieldEffectSettingsProxy> PrivateGetProxy() const { return GetNewProxy(); }

	// List of classes that use USoundfieldEffectSettingsBase:
	friend Audio::FMixerSubmix;
	friend Audio::FMixerSourceManager;
};

/**
 * Single instance that actually processes the soundfield.
 */
class ISoundfieldEffectInstance
{
public:
	virtual ~ISoundfieldEffectInstance() = default;

	virtual void ProcessAudio(ISoundfieldAudioPacket& InOutPacket, const ISoundfieldEncodingSettingsProxy& EncodingSettings, const ISoundfieldEffectSettingsProxy& ProcessorSettings) = 0;
};

/**
 * This opaque class should be used for specifying settings for how audio should be encoded
 * to your soundfield format for a given submix or file.
 */
UCLASS(config = Engine, abstract, editinlinenew, BlueprintType, MinimalAPI)
class USoundfieldEffectBase : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * TODO: Filter classes settable on here by GetSettingsClass.
	 */
	UPROPERTY(EditAnywhere, Category = EffectPreset)
	TObjectPtr<USoundfieldEffectSettingsBase> Settings;

protected:
	/*
	 * Get the implementation of USoundfieldProcessorSettingsBase that is used for this processor's settings. Will always be called on the CDO.
	 */
	virtual bool SupportsFormat(const FName& InFormat) const PURE_VIRTUAL(USoundfieldEncodingSettingsBase::SupportsFormat, return false;);

	/*
	 * Get the implementation of USoundfieldProcessorSettingsBase that is used for this processor's settings. Will always be called on the CDO.
	 */
	virtual const UClass* GetSettingsClass() const PURE_VIRTUAL(USoundfieldEncodingSettingsBase::GetSettingsClass, return nullptr;);
	
	/**
	 * return the default processor settings we should use when none is provided. Will always be called on the CDO.
	 */
	virtual const USoundfieldEffectSettingsBase* GetDefaultSettings() const PURE_VIRTUAL(USoundfieldEncodingSettingsBase::GetDefaultSettings, return nullptr;);

	/**
	 * Spawn a new instance of this processor.
	 */
	virtual TUniquePtr<ISoundfieldEffectInstance> GetNewProcessor(const ISoundfieldEncodingSettingsProxy& EncodingSettings) const PURE_VIRTUAL(USoundfieldEncodingSettingsBase::GetProxy, return nullptr;);

private:

	const USoundfieldEffectSettingsBase* PrivateGetDefaultSettings() const { return GetDefaultSettings(); };
	TUniquePtr<ISoundfieldEffectInstance> PrivateGetNewProcessor(const ISoundfieldEncodingSettingsProxy& EncodingSettings) const { return GetNewProcessor(EncodingSettings); }

	// List of classes that use USoundfieldEffectBase:
	friend Audio::FMixerSourceManager;
	friend Audio::FMixerSubmix;

};

/** This is used in FMixerSourceVoice to make sure we only encode sources once for each type of stream.  */
struct FSoundfieldEncodingKey
{
	FName SoundfieldFormat;
	int32 EncodingSettingsID;

	FSoundfieldEncodingKey()
		: SoundfieldFormat(ISoundfieldFactory::GetFormatNameForNoEncoding())
		, EncodingSettingsID(0)
	{
	}

	FSoundfieldEncodingKey(ISoundfieldFactory* Factory, ISoundfieldEncodingSettingsProxy& InSettings)
		: SoundfieldFormat(Factory ? Factory->GetSoundfieldFormatName() : ISoundfieldFactory::GetFormatNameForNoEncoding())
		, EncodingSettingsID(InSettings.GetUniqueId())
	{
	}

	inline bool operator==(const FSoundfieldEncodingKey& Other) const
	{
		return (SoundfieldFormat == Other.SoundfieldFormat) && (EncodingSettingsID == Other.EncodingSettingsID);
	} 

	friend inline uint32 GetTypeHash(const FSoundfieldEncodingKey& Value)
	{
		return HashCombine(Value.EncodingSettingsID, Value.SoundfieldFormat.GetNumber());
	}
};
