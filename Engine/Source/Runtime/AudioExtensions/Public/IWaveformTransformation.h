// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "DSP/BufferVectorOperations.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "IWaveformTransformation.generated.h"

class USoundWave;

namespace Audio
{
	// information about the current state of the wave file we are transforming
	struct FWaveformTransformationWaveInfo
	{
		float SampleRate = 0.f;
		int32 NumChannels = 0;
		Audio::FAlignedFloatBuffer* Audio = nullptr;
		int32 StartSampleOffset = 0;
		int32 NumEditedSamples = 0;
	};

	enum class ETransformationPriority : uint8
	{
		None = 0,
		Low,
		High
	};

	/*
	 * Base class for the object that processes waveform data
	 * Pass tweakable variables from its paired settings UObject in the constructor in UWaveformTransformationBase::CreateTransformation
	 *
	 * note: WaveTransformation vs WaveformTransformation is to prevent UHT class name conflicts without having to namespace everything - remember this in derived classes!
	 */
	class IWaveTransformation
	{
	public:

		// Applies the transformation to the waveform and modifies WaveInfo with the resulting changes
		virtual void ProcessAudio(FWaveformTransformationWaveInfo& InOutWaveInfo) const {};
		
		virtual bool SupportsRealtimePreview() const { return false; }
		virtual constexpr ETransformationPriority FileChangeLengthPriority() const { return ETransformationPriority::None; }
		virtual bool CanChangeChannelCount() const { return false; }

		virtual ~IWaveTransformation() = default;
	};

	using FTransformationPtr = TUniquePtr<Audio::IWaveTransformation>;
}

// Struct defining a cue point in a sound wave asset
USTRUCT(BlueprintType)
struct FSoundWaveCuePoint
{
	GENERATED_USTRUCT_BODY()

	// Unique identifier for the wave cue point
	UPROPERTY(Category = Info, VisibleAnywhere, BlueprintReadOnly)
	int32 CuePointID = INDEX_NONE;

	// The label for the cue point
	UPROPERTY(Category = Info, EditAnywhere, BlueprintReadOnly)
	FString Label;

	// The frame position of the cue point
	UPROPERTY(Category = Info, EditAnywhere, BlueprintReadOnly, meta = (ClampMin = 0))
	int64 FramePosition = 0;

	// The frame length of the cue point (non-zero if it's a region)
	UPROPERTY(Category = Info, EditAnywhere, BlueprintReadOnly, meta = (ClampMin = 0))
	int64 FrameLength = 0;

	bool IsLoopRegion() const { return bIsLoopRegion; }
	void SetLoopRegion(bool value) { bIsLoopRegion = value; }

	const bool operator==(const FSoundWaveCuePoint& Other) const
	{ 
		return CuePointID == Other.CuePointID &&
			FrameLength == Other.FrameLength &&
			FramePosition == Other.FramePosition &&
			Label == Other.Label &&
			IsLoopRegion() == Other.IsLoopRegion();
	}

	const bool operator!=(const FSoundWaveCuePoint& Other) const { return !(*this == Other); }

#if WITH_EDITORONLY_DATA
	void ScaleFrameValues(float Factor)
	{
		FramePosition = FMath::FloorToInt64((float)FramePosition * Factor);
		FrameLength = FMath::FloorToInt64((float)FrameLength * Factor);
	}
#endif // WITH_EDITORONLY_DATA

	friend class USoundFactory;
	friend class USoundWave;

protected:
	// intentionally kept private.
	// only USoundFactory should modify this value on import
	UPROPERTY(Category = Info, EditAnywhere, BlueprintReadOnly)
	bool bIsLoopRegion = false;
};

// Information about the the wave file we are transforming for Transformation UObjects
struct FWaveTransformUObjectConfiguration
{
	int32 NumChannels = 0;
	float SampleRate = 0;
	float StartTime = 0.f; 
	float EndTime = -1.f; 
	TArray<FSoundWaveCuePoint> WaveCues; // List of cues parsed from the wave file
	bool bIsPreviewingLoopRegion = false;
	bool bCachedSoundWaveLoopState = false;

protected:
	int64 OriginalNumFrames = 0;

public:
	void SetOriginalNumFrames(const int64 InOriginalNumFrames) { OriginalNumFrames = InOriginalNumFrames; }
	const int64 GetOriginalNumFrames() const { return OriginalNumFrames; }

	int64 GetStartFrameOffset() const { return FMath::RoundToInt64(StartTime * SampleRate); }
	AUDIOEXTENSIONS_API int64 GetEndFrameOffset() const;
};

#if WITH_EDITORONLY_DATA
// Information to be retrieved from each transformation (Add members as needed)
struct FWaveformTransformationInfo
{
	TArray<FSoundWaveCuePoint> AllCuePoints; // Cue Points and Loop Regions
	TArray<FSoundWaveCuePoint> AllCuePointsRelativeToStartTime; // Cue Points and Loop Regions offset by start time
};
#endif //WITH_EDITORONLY_DATA

// Base class to hold editor configurable properties for an arbitrary transformation of audio waveform data
UCLASS(Abstract, EditInlineNew, MinimalAPI)
class UWaveformTransformationBase : public UObject
{
	GENERATED_BODY()

public:
	virtual Audio::FTransformationPtr CreateTransformation() const { return nullptr; }
	virtual void UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration) {};
	virtual void OverwriteTransformation() {};
	virtual constexpr Audio::ETransformationPriority GetTransformationPriority() const { return Audio::ETransformationPriority::None; }

	// Sort to ensure proper order of operation for audio processing
	static void SortTransformationsArray(TArray<TObjectPtr<UWaveformTransformationBase>>& InOutTransformations) 
	{
		Algo::StableSort(InOutTransformations, [](const TObjectPtr<UWaveformTransformationBase>& TransformationA, const TObjectPtr<UWaveformTransformationBase>& TransformationB)
			{
				if ((!TransformationA && TransformationB) || (!TransformationA && !TransformationB))
				{
					return false;
				}
				else if (TransformationA && !TransformationB)
				{
					return true;
				}

				return TransformationA->GetTransformationPriority() > TransformationB->GetTransformationPriority();
			});
	}
	
#if WITH_EDITOR
	virtual void OverwriteSoundWaveData(USoundWave& InOutSoundWave) {};
	virtual void GetTransformationInfo(FWaveformTransformationInfo& InOutTransformationInfo) const {};
	virtual FString GetTransformationHash() const PURE_VIRTUAL(UWaveformTransformationBase::GetTransformationHash, return FString(););

	AUDIOEXTENSIONS_API virtual void PostEditUndo() override;
	AUDIOEXTENSIONS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	void NotifyPropertyChange(FProperty* Property);
#endif //WITH_EDITOR

	virtual bool IsEditorOnly() const override { return true; }


	// Execute when the soundwave needs a recook to apply new transformation changes, bool flag if it should mark the file dirty
	DECLARE_DELEGATE_OneParam(FOnTransformationChanged, bool);
	FOnTransformationChanged OnTransformationChanged;

	// Execute when only the rendering needs to be updated, but can wait for a recook
	DECLARE_DELEGATE(FOnTransformationRenderChanged);
	FOnTransformationRenderChanged OnTransformationRenderChanged;
};

// Object that holds an ordered list of transformations to perform on a sound wave
UCLASS(EditInlineNew, MinimalAPI)
class UWaveformTransformationChain : public UObject
{
	GENERATED_BODY()
	
public:
	UPROPERTY(EditAnywhere, Instanced, Category = "Transformations")
	TArray<TObjectPtr<UWaveformTransformationBase>> Transformations;

	virtual bool IsEditorOnly() const override { return true; }
	
	AUDIOEXTENSIONS_API TArray<Audio::FTransformationPtr> CreateTransformations() const;
};
