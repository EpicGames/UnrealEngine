// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SoundGeneratorOutput.generated.h"

/**
 * Base class for generators that have outputs that can be exposed to other game code.
 *
 * NOTE: This is not widely supported and should be considered experimental.
 */
USTRUCT(BlueprintType)
struct FSoundGeneratorOutput
{
	GENERATED_BODY()

	/** The output's name */
	UPROPERTY(BlueprintReadOnly, Category = SoundGeneratorOutput)
	FName Name;
};
